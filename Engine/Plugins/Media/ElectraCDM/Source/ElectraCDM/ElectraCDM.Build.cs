// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ElectraCDM : ModuleRules
	{
		public ElectraCDM(ReadOnlyTargetRules Target) : base(Target)
		{
			IWYUSupport = IWYUSupport.None; // Disabled because of third party code
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"Json"
				});
		}
	}
}
