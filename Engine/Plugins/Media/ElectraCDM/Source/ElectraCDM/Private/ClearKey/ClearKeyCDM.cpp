// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClearKey/ClearKeyCDM.h"
#include "ElectraCDM.h"
#include "ElectraCDMClient.h"
#include "ElectraCDMUtils.h"
#include "Crypto/StreamCryptoAES128.h"
#include <Misc/Base64.h>
#include <Misc/ScopeLock.h>
#include <Dom/JsonObject.h>
#include <Serialization/JsonReader.h>
#include <Serialization/JsonSerializer.h>

#define ENABLE_LEGACY_RAWKEY_OVERRIDE 1

using namespace ElectraCDMUtils;

namespace ElectraCDM
{

namespace ClearKeyCDM
{
static const TCHAR* const JSONTextPropertyName = TEXT("#text");
static FString UrlAttrib_dashif_laurl(TEXT("dashif:laurl"));
static FString UrlAttrib_laurl(TEXT("laurl"));
static FString UrlAttrib_clearkey_laurl(TEXT("clearkey:Laurl"));
static FString UrlAttrib_Laurl(TEXT("Laurl"));
};

enum EClearKeyScheme
{
	Unsupported,
	Cenc,
	Cbcs
};

struct FClearKeyKIDKey
{
	TArray<uint8> KID;
	TArray<uint8> Key;
	EClearKeyScheme EncryptionScheme = EClearKeyScheme::Unsupported;
};


class FClearKeyCDM : public IClearKeyCDM, public IMediaCDMCapabilities, public TSharedFromThis<FClearKeyCDM, ESPMode::ThreadSafe>
{
public:
	FClearKeyCDM() = default;
	virtual ~FClearKeyCDM() = default;
	virtual FString GetLastErrorMessage() override;
	virtual const TArray<FString>& GetSchemeIDs() override;
	virtual void GetCDMCustomJSONPrefixes(FString& OutAttributePrefix, FString& OutTextPropertyName, bool& bOutNoNamespaces) override;
	virtual TSharedPtr<IMediaCDMCapabilities, ESPMode::ThreadSafe> GetCDMCapabilities(const FString& InValue, const FString& InAdditionalElements) override;
	virtual ECDMError CreateDRMClient(TSharedPtr<IMediaCDMClient, ESPMode::ThreadSafe>& OutClient, IMediaCDM::IPlayerSession* InForPlayerSession, const TArray<IMediaCDM::FCDMCandidate>& InCandidates) override;
	virtual ECDMError ReleasePlayerSessionKeys(IMediaCDM::IPlayerSession* PlayerSession) override;

	virtual ESupportResult SupportsCipher(const FString& InCipherType) override;
	virtual ESupportResult SupportsType(const FString& InMimeType) override;
	virtual ESupportResult RequiresSecureDecoder(const FString& InMimeType) override;

	void AddPlayerSessionKeys(IMediaCDM::IPlayerSession* InPlayerSession, const TArray<FClearKeyKIDKey>& InNewSessionKeys);
	bool GetPlayerSessionKey(FClearKeyKIDKey& OutKey, IMediaCDM::IPlayerSession* InPlayerSession, const TArray<uint8>& InForKID);

	static EClearKeyScheme GetCommonSchemeFromCipher(const FString& InCipherName);
	static EClearKeyScheme GetCommonSchemeFromCipher(uint32 InCipher4CC);
public:
	static TSharedPtr<FClearKeyCDM, ESPMode::ThreadSafe> Get();

	FCriticalSection Lock;
	TMap<IMediaCDM::IPlayerSession*, TArray<FClearKeyKIDKey>> ActiveLicensesPerPlayer;
	FString LastErrorMessage;
};


class FClearKeyDRMDecrypter : public IMediaCDMDecrypter
{
public:
	FClearKeyDRMDecrypter();
	void Initialize(const FString& InMimeType);
	void SetLicenseKeys(const TArray<FClearKeyKIDKey>& InLicenseKeys);
	void SetState(ECDMState InNewState);
	void SetLastErrorMessage(const FString& InNewErrorMessage);
	virtual ~FClearKeyDRMDecrypter();
	virtual ECDMState GetState() override;
	virtual FString GetLastErrorMessage() override;
	virtual ECDMError UpdateInitDataFromPSSH(const TArray<uint8>& InPSSHData) override;
	virtual ECDMError UpdateInitDataFromMultiplePSSH(const TArray<TArray<uint8>>& InPSSHData) override;
	virtual ECDMError UpdateFromURL(const FString& InURL, const FString& InAdditionalElements) override;
	virtual bool IsBlockStreamDecrypter() override;
	virtual void Reinitialize() override;
	virtual ECDMError DecryptInPlace(uint8* InOutData, int32 InNumDataBytes, const FMediaCDMSampleInfo& InSampleInfo) override;
	virtual ECDMError BlockStreamDecryptStart(IStreamDecryptHandle*& OutStreamDecryptContext, const FMediaCDMSampleInfo& InSampleInfo) override;
	virtual ECDMError BlockStreamDecryptInPlace(IStreamDecryptHandle* InOutStreamDecryptContext, int32& OutNumBytesDecrypted, uint8* InOutData, int32 InNumDataBytes, bool bIsLastBlock) override;
	virtual ECDMError BlockStreamDecryptEnd(IStreamDecryptHandle* InStreamDecryptContext) override;

private:
	struct FKeyDecrypter
	{
		FClearKeyKIDKey KIDKey;
		TSharedPtr<ElectraCDM::IStreamDecrypterAES128, ESPMode::ThreadSafe> Decrypter;
		ECDMState State = ECDMState::Idle;
		bool bIsInitialized = false;
	};

	FClearKeyDRMDecrypter::FKeyDecrypter*  GetDecrypterForKID(const TArray<uint8>& KID);

	FCriticalSection Lock;
	TArray<FClearKeyKIDKey> LicenseKeys;
	TArray<FKeyDecrypter> KeyDecrypters;
	ECDMState CurrentState = ECDMState::Idle;
	FString LastErrorMsg;
};


class FClearKeyDRMClient : public IMediaCDMClient, public TSharedFromThis<FClearKeyDRMClient, ESPMode::ThreadSafe>
{
public:
	virtual ~FClearKeyDRMClient();

	FClearKeyDRMClient();
	void Initialize(TSharedPtr<FClearKeyCDM, ESPMode::ThreadSafe> InOwningCDM, IMediaCDM::IPlayerSession* InForPlayerSession, const TArray<IMediaCDM::FCDMCandidate>& InCDMConfigurations);

	virtual ECDMState GetState() override;
	virtual FString GetLastErrorMessage() override;

	virtual void RegisterEventListener(TWeakPtr<IMediaCDMEventListener, ESPMode::ThreadSafe> InEventListener) override;
	virtual void UnregisterEventListener(TWeakPtr<IMediaCDMEventListener, ESPMode::ThreadSafe> InEventListener) override;
	virtual void PrepareLicenses() override;
	virtual void SetLicenseServerURL(const FString& InLicenseServerURL) override;
	virtual void GetLicenseKeyURL(FString& OutLicenseURL) override;
	virtual void GetLicenseKeyRequestData(TArray<uint8>& OutKeyRequestData, FString& OutHttpMethod, TArray<FString>& OutHttpHeaders, uint32& OutFlags) override;
	virtual ECDMError SetLicenseKeyResponseData(void* InEventId, int32 HttpResponseCode, const TArray<uint8>& InKeyResponseData) override;
	virtual ECDMError CreateDecrypter(TSharedPtr<IMediaCDMDecrypter, ESPMode::ThreadSafe>& OutDecrypter, const FString& InMimeType) override;

private:
	EClearKeyScheme GetCommonSchemeFromConfiguration(const IMediaCDM::FCDMCandidate& InConfiguration);
	TArray<IMediaCDM::FCDMCandidate> GetConfigurationsForKID(const TArray<uint8>& InForKID);
	int32 PrepareKIDsToRequest();
	void AddKeyKID(const FClearKeyKIDKey& InKeyKid);
	void AddKeyKIDs(const TArray<FClearKeyKIDKey>& InKeyKids);
	void FireEvent(IMediaCDMEventListener::ECDMEventType InEvent);
	void RemoveStaleDecrypters();
	void UpdateKeyWithDecrypters();
	void UpdateStateWithDecrypters(ECDMState InNewState);
	void GetValuesFromConfigurations();
	bool GetURLsFrom(TArray<FString>& OutURLs, const TSharedPtr<FJsonValue>& JSONValue, const FString& PropertyName, const FString& AltPropertyName);
	bool GetURLsFrom(TArray<FString>& OutURLs, const TSharedPtr<FJsonObject>& JSON, const FString& PropertyName, const FString& AltPropertyName);

	FCriticalSection Lock;
	IMediaCDM::IPlayerSession* PlayerSession = nullptr;
	TWeakPtr<FClearKeyCDM, ESPMode::ThreadSafe> OwningCDM;
	TArray<IMediaCDM::FCDMCandidate> CDMConfigurations;
	TArray<TWeakPtr<IMediaCDMEventListener, ESPMode::ThreadSafe>> Listeners;
	TArray<TWeakPtr<FClearKeyDRMDecrypter, ESPMode::ThreadSafe>> Decrypters;
	TArray<FString> PendingRequiredKIDs;
	TArray<FClearKeyKIDKey> LicenseKeys;
	TOptional<FString> LicenseServerURLOverride;
	TArray<FString> LicenseServerURLsFromConfigs;
	ECDMState CurrentState = ECDMState::Idle;
	FString LastErrorMsg;
};

/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/


//-----------------------------------------------------------------------------
/**
 * Registers this ClearKey CDM with the CDM manager
 */
void IClearKeyCDM::RegisterWith(IMediaCDM& InDRMManager)
{
	InDRMManager.RegisterCDM(FClearKeyCDM::Get());
}

//-----------------------------------------------------------------------------
/**
 * Returns the singleton of this CDM system.
 */
TSharedPtr<FClearKeyCDM, ESPMode::ThreadSafe> FClearKeyCDM::Get()
{
	static TSharedPtr<FClearKeyCDM, ESPMode::ThreadSafe> This = MakeShared<FClearKeyCDM, ESPMode::ThreadSafe>();
	return This;
}

//-----------------------------------------------------------------------------
/**
 * Returns the scheme ID of ClearKey.
 * This is the official ID used with DASH.
 */
const TArray<FString>& FClearKeyCDM::GetSchemeIDs()
{
	static TArray<FString> SchemeIDs({TEXT("e2719d58-a985-b3c9-781a-b030af78d30e"),			// DASH
									  TEXT("1077efec-c0b2-4d02-ace3-3c1e52e2fb4b")			// W3C
									 });
	return SchemeIDs;
}

//-----------------------------------------------------------------------------
/**
 * Returns the most recent error message.
 */
FString FClearKeyCDM::GetLastErrorMessage()
{
	FScopeLock lock(&Lock);
	return LastErrorMessage;
}

//-----------------------------------------------------------------------------
/**
 * Returns the expected element prefixes for the AdditionalElements JSON.
 */
void FClearKeyCDM::GetCDMCustomJSONPrefixes(FString& OutAttributePrefix, FString& OutTextPropertyName, bool& bOutNoNamespaces)
{
	OutAttributePrefix.Empty();
	OutTextPropertyName = ClearKeyCDM::JSONTextPropertyName;
	bOutNoNamespaces = false;	// keep the namespaces to differentiate between dashif:laurl and clearkey:Laurl
}

//-----------------------------------------------------------------------------
/**
 * Returns the capability interface of this CDM.
 */
TSharedPtr<IMediaCDMCapabilities, ESPMode::ThreadSafe> FClearKeyCDM::GetCDMCapabilities(const FString& InValue, const FString& InAdditionalElements)
{
	if (InValue.IsEmpty() || InValue.Equals(TEXT("ClearKey1.0")))
	{
		return AsShared();
	}
	return nullptr;
}

//-----------------------------------------------------------------------------
/**
 * Creates a client instance of this CDM.
 * The application may create one or more instances, possibly for different key IDs.
 * In the context of DASH there is to be one client per Period that contains one or
 * more AdaptationSets that are encrypted.
 */
ECDMError FClearKeyCDM::CreateDRMClient(TSharedPtr<IMediaCDMClient, ESPMode::ThreadSafe>& OutClient, IMediaCDM::IPlayerSession* InForPlayerSession, const TArray<IMediaCDM::FCDMCandidate>& InCandidates)
{
	FClearKeyDRMClient* NewClient = new FClearKeyDRMClient;
	NewClient->Initialize(AsShared(), InForPlayerSession, InCandidates);
	OutClient = MakeShareable(NewClient);
	FScopeLock lock(&Lock);
	LastErrorMessage.Empty();
	return ECDMError::Success;
}

//-----------------------------------------------------------------------------
/**
 * Adds keys to the specified player session.
 */
void FClearKeyCDM::AddPlayerSessionKeys(IMediaCDM::IPlayerSession* InPlayerSession, const TArray<FClearKeyKIDKey>& InNewSessionKeys)
{
	FScopeLock lock(&Lock);
	TArray<FClearKeyKIDKey>& Keys = ActiveLicensesPerPlayer.FindOrAdd(InPlayerSession);
	for(auto &NewKey : InNewSessionKeys)
	{
		bool bHaveAlready = false;
		for(auto &HaveKey : Keys)
		{
			if (HaveKey.KID == NewKey.KID)
			{
				bHaveAlready = true;
				break;
			}
		}
		if (!bHaveAlready)
		{
			Keys.Emplace(NewKey);
		}
	}
}

//-----------------------------------------------------------------------------
/**
 * Returns a player session's key for the specified KID.
 */
bool FClearKeyCDM::GetPlayerSessionKey(FClearKeyKIDKey& OutKey, IMediaCDM::IPlayerSession* InPlayerSession, const TArray<uint8>& InForKID)
{
	FScopeLock lock(&Lock);
	const TArray<FClearKeyKIDKey>* Keys = ActiveLicensesPerPlayer.Find(InPlayerSession);
	if (Keys)
	{
		for(auto &Key : *Keys)
		{
			if (Key.KID == InForKID)
			{
				OutKey = Key;
				return true;
			}
		}
	}
	return false;
}

//-----------------------------------------------------------------------------
/**
 * Releases all keys the specified player session has acquired.
 */
ECDMError FClearKeyCDM::ReleasePlayerSessionKeys(IMediaCDM::IPlayerSession* PlayerSession)
{
	FScopeLock lock(&Lock);
	LastErrorMessage.Empty();
	ActiveLicensesPerPlayer.Remove(PlayerSession);
	return ECDMError::Success;
}


//-----------------------------------------------------------------------------
/**
 * Returns if a specified cipher (eg. "cenc" or "cbcs") is supported by this CDM.
 */
IMediaCDMCapabilities::ESupportResult FClearKeyCDM::SupportsCipher(const FString& InCipherType)
{
	return GetCommonSchemeFromCipher(InCipherType) != EClearKeyScheme::Unsupported ? IMediaCDMCapabilities::ESupportResult::Supported : IMediaCDMCapabilities::ESupportResult::NotSupported;
}

//-----------------------------------------------------------------------------
/**
 * Returns if a media stream of a given format can be decrypted with this CDM.
 * The mime type should include a codecs="..." component and if it is video it
 * should also have a resolution=...x... component.
 */
IMediaCDMCapabilities::ESupportResult FClearKeyCDM::SupportsType(const FString& InMimeType)
{
	// Everything is supported.
	return IMediaCDMCapabilities::ESupportResult::Supported;
}

//-----------------------------------------------------------------------------
/**
 * Returns whether or not for a particular media stream format a secure decoder is
 * required to be used.
 */
IMediaCDMCapabilities::ESupportResult FClearKeyCDM::RequiresSecureDecoder(const FString& InMimeType)
{
	// This is ClearKey... there is no real security here to begin with.
	return IMediaCDMCapabilities::ESupportResult::SecureDecoderNotRequired;
}

//-----------------------------------------------------------------------------
/**
 * Converts cipher name to enum.
 */
EClearKeyScheme FClearKeyCDM::GetCommonSchemeFromCipher(const FString& InCipherName)
{
	if (InCipherName.Equals(TEXT("cenc"), ESearchCase::IgnoreCase))
	{
		return EClearKeyScheme::Cenc;
	}
	else if (InCipherName.Equals(TEXT("cbcs"), ESearchCase::IgnoreCase))
	{
		return EClearKeyScheme::Cbcs;
	}
	return EClearKeyScheme::Unsupported;
}

EClearKeyScheme FClearKeyCDM::GetCommonSchemeFromCipher(uint32 InCipher4CC)
{
	#define MAKE_4CC(a,b,c,d) (uint32)(((uint32)a << 24) | ((uint32)b << 16) | ((uint32)c << 8) | ((uint32)d))
	if (InCipher4CC == MAKE_4CC('c', 'e', 'n', 'c'))
	{
		return EClearKeyScheme::Cenc;
	}
	else if (InCipher4CC == MAKE_4CC('c', 'b', 'c', 's'))
	{
		return EClearKeyScheme::Cbcs;
	}
	return EClearKeyScheme::Unsupported;
}


/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/


//-----------------------------------------------------------------------------
/**
 * Construct a new client
 */
FClearKeyDRMClient::FClearKeyDRMClient()
{
}

//-----------------------------------------------------------------------------
/**
 * Destroy a client
 */
FClearKeyDRMClient::~FClearKeyDRMClient()
{
}

//-----------------------------------------------------------------------------
/**
 * Returns the client's current state.
 */
ECDMState FClearKeyDRMClient::GetState()
{
	FScopeLock lock(&Lock);
	return CurrentState;
}

//-----------------------------------------------------------------------------
/**
 * Returns the client's most recent error message.
 */
FString FClearKeyDRMClient::GetLastErrorMessage()
{
	FScopeLock lock(&Lock);
	return LastErrorMsg;
}

//-----------------------------------------------------------------------------
/**
 * Initializes the client.
 */
void FClearKeyDRMClient::Initialize(TSharedPtr<FClearKeyCDM, ESPMode::ThreadSafe> InOwningCDM, IMediaCDM::IPlayerSession* InForPlayerSession, const TArray<IMediaCDM::FCDMCandidate>& InCDMConfigurations)
{
	FScopeLock lock(&Lock);
	OwningCDM = InOwningCDM;
	PlayerSession = InForPlayerSession;
	CDMConfigurations = InCDMConfigurations;
	GetValuesFromConfigurations();
	CurrentState = ECDMState::Idle;
}

//-----------------------------------------------------------------------------
/**
 * Registers an event listener to the client.
 */
void FClearKeyDRMClient::RegisterEventListener(TWeakPtr<IMediaCDMEventListener, ESPMode::ThreadSafe> InEventListener)
{
	FScopeLock lock(&Lock);
	Listeners.Emplace(InEventListener);

	// Based on the current state we may need to fire events to the new listener right away.
	if (CurrentState == ECDMState::WaitingForKey)
	{
		lock.Unlock();
		FireEvent(IMediaCDMEventListener::ECDMEventType::KeyRequired);
	}
}

//-----------------------------------------------------------------------------
/**
 * Unregisters an event listener from the client.
 */
void FClearKeyDRMClient::UnregisterEventListener(TWeakPtr<IMediaCDMEventListener, ESPMode::ThreadSafe> InEventListener)
{
	FScopeLock lock(&Lock);
	Listeners.Remove(InEventListener);
}

//-----------------------------------------------------------------------------
/**
 * Fires the given event at all registered event listeners.
 */
void FClearKeyDRMClient::FireEvent(IMediaCDMEventListener::ECDMEventType InEvent)
{
	TArray<TWeakPtr<IMediaCDMEventListener, ESPMode::ThreadSafe>> CopiedListeners;
	Lock.Lock();
	for(int32 i=0; i<Listeners.Num(); ++i)
	{
		if (Listeners[i].IsValid())
		{
			CopiedListeners.Emplace(Listeners[i]);
		}
		else
		{
			Listeners.RemoveAt(i);
			--i;
		}
	}
	Lock.Unlock();
	TArray<uint8> NoData;
	TSharedPtr<ElectraCDM::IMediaCDMClient, ESPMode::ThreadSafe> This = AsShared();
	for(int32 i=0; i<CopiedListeners.Num(); ++i)
	{
		TSharedPtr<IMediaCDMEventListener, ESPMode::ThreadSafe> Listener = CopiedListeners[i].Pin();
		if (Listener.IsValid())
		{
			Listener->OnCDMEvent(InEvent, This, nullptr, NoData);
		}
	}
}

//-----------------------------------------------------------------------------
/**
 * Prepares the client to fetch a license and fires the event off to the
 * listeners to start the process.
 */
void FClearKeyDRMClient::PrepareLicenses()
{
	int32 NumToRequest = PrepareKIDsToRequest();
	if (NumToRequest)
	{
		Lock.Lock();
		CurrentState = ECDMState::WaitingForKey;
		Lock.Unlock();
		FireEvent(IMediaCDMEventListener::ECDMEventType::KeyRequired);
	}
	else
	{
		Lock.Lock();
		CurrentState = ECDMState::Ready;
		Lock.Unlock();
	}
}

//-----------------------------------------------------------------------------
/**
 * Overrides the license server URL to the given one.
 * This must happen before calling PrepareLicenses().
 */
void FClearKeyDRMClient::SetLicenseServerURL(const FString& InLicenseServerURL)
{
	FScopeLock lock(&Lock);
	LicenseServerURLOverride = InLicenseServerURL;
}

//-----------------------------------------------------------------------------
/**
 * Returns the license server URL to which to issue the license request.
 */
void FClearKeyDRMClient::GetLicenseKeyURL(FString& OutLicenseURL)
{
	FScopeLock lock(&Lock);
	// If the URL has been set explicity from the outside return that one.
	if (LicenseServerURLOverride.IsSet())
	{
		OutLicenseURL = LicenseServerURLOverride.GetValue();
		return;
	}
	// Otherwise, when there are several specified through the AdditionalElements
	// we can return one of them at random.
	// Which means we take the first one.
	if (LicenseServerURLsFromConfigs.Num())
	{
		OutLicenseURL = LicenseServerURLsFromConfigs[0];
		return;
	}
	// Nothing set at all. Clear out the URL in case it contains something.
	OutLicenseURL.Empty();
}

//-----------------------------------------------------------------------------
/**
 * Adds a new KID with license key if the KID is not already known.
 */
void FClearKeyDRMClient::AddKeyKID(const FClearKeyKIDKey& InKeyKid)
{
	FScopeLock lock(&Lock);
	for(auto &Key : LicenseKeys)
	{
		if (Key.KID == InKeyKid.KID)
		{
			return;
		}
	}
	LicenseKeys.Add(InKeyKid);
}

//-----------------------------------------------------------------------------
/**
 * Adds a list of new KIDs with license keys when the KID is not already known.
 */
void FClearKeyDRMClient::AddKeyKIDs(const TArray<FClearKeyKIDKey>& InKeyKids)
{
	for(auto &KeyKid : InKeyKids)
	{
		AddKeyKID(KeyKid);
	}
}

//-----------------------------------------------------------------------------
/**
 * Prepares the list of KIDs for which a license must be obtained.
 * Licenses the CDM already has will not be requested again.
 */
int32 FClearKeyDRMClient::PrepareKIDsToRequest()
{
	// We need to get all the KIDs for which we (may) need to acquire a license.
	FScopeLock lock(&Lock);
	PendingRequiredKIDs.Empty();
	check(CDMConfigurations.Num());
	TSharedPtr<FClearKeyCDM, ESPMode::ThreadSafe> CDM = OwningCDM.Pin();
	if (CDMConfigurations.Num())
	{
		for(int32 nCfg=0; nCfg<CDMConfigurations.Num(); ++nCfg)
		{
			for(int32 nKIDs=0; nKIDs<CDMConfigurations[nCfg].DefaultKIDs.Num(); ++nKIDs)
			{
				if (CDMConfigurations[nCfg].DefaultKIDs[nKIDs].Len())
				{
					// Check if the CDM already has a key for this session's KID.
					FString KID = StripDashesFromKID(CDMConfigurations[nCfg].DefaultKIDs[nKIDs]);
					TArray<uint8> BinKID;
					FClearKeyKIDKey KeyKid;
					ConvertKIDToBin(BinKID, KID);
					if (CDM.IsValid() && CDM->GetPlayerSessionKey(KeyKid, PlayerSession, BinKID))
					{
						AddKeyKID(KeyKid);
						continue;
					}
					PendingRequiredKIDs.AddUnique(ConvertKIDToBase64(KID));
				}
			}
		}
	}
	return PendingRequiredKIDs.Num();
}


//-----------------------------------------------------------------------------
/**
 * Converts encryption scheme string to enum.
 */
EClearKeyScheme FClearKeyDRMClient::GetCommonSchemeFromConfiguration(const IMediaCDM::FCDMCandidate& InConfiguration)
{
	return FClearKeyCDM::GetCommonSchemeFromCipher(InConfiguration.CommonScheme);
}


//-----------------------------------------------------------------------------
/**
 * Returns the CDM configuration objects matching the given KID.
 */
TArray<IMediaCDM::FCDMCandidate> FClearKeyDRMClient::GetConfigurationsForKID(const TArray<uint8>& InForKID)
{
	TArray<IMediaCDM::FCDMCandidate> Cfgs;
	FScopeLock lock(&Lock);
	for(int32 nCfg=0; nCfg<CDMConfigurations.Num(); ++nCfg)
	{
		for(int32 nKIDs=0; nKIDs<CDMConfigurations[nCfg].DefaultKIDs.Num(); ++nKIDs)
		{
			if (CDMConfigurations[nCfg].DefaultKIDs[nKIDs].Len())
			{
				TArray<uint8> BinKID;
				ConvertKIDToBin(BinKID, StripDashesFromKID(CDMConfigurations[nCfg].DefaultKIDs[nKIDs]));
				if (BinKID == InForKID)
				{
					Cfgs.Emplace(CDMConfigurations[nCfg]);
				}
			}
		}
	}
	return Cfgs;
}

//-----------------------------------------------------------------------------
/**
 * Returns the information necessary to make the license request.
 * This includes the system specific blob of data as well as the HTTP method
 * to use and additioanl headers, like the "Content-Type: " header.
 */
void FClearKeyDRMClient::GetLicenseKeyRequestData(TArray<uint8>& OutKeyRequestData, FString& OutHttpMethod, TArray<FString>& OutHttpHeaders, uint32& OutFlags)
{
	FScopeLock lock(&Lock);
	// The request is a JSON string like here: https://w3c.github.io/encrypted-media/index.html#example-1
	FString rq1 = TEXT("{\"kids\":[");
	FString rq2 = TEXT("],\"type\":\"temporary\"}");
	FString RequestJSON = rq1;
	for(int32 i=0; i<PendingRequiredKIDs.Num(); ++i)
	{
		RequestJSON += FString::Printf(TEXT("\"%s\""), *PendingRequiredKIDs[i]);
		if (i+1 < PendingRequiredKIDs.Num())
		RequestJSON.AppendChar(TCHAR(','));
	}
	RequestJSON += rq2;

	OutHttpMethod = TEXT("POST");
	OutHttpHeaders.Emplace(TEXT("Content-Type: application/json"));
	StringToArray(OutKeyRequestData, RequestJSON);
	// We allow the use of custom key storage in case there is no license server URL.
	OutFlags = EDRMClientFlags::EDRMFlg_AllowCustomKeyStorage;
}

//-----------------------------------------------------------------------------
/**
 * Parses the license key response for keys and provides them to the
 * decrypter instances.
 */
ECDMError FClearKeyDRMClient::SetLicenseKeyResponseData(void* InEventId, int32 HttpResponseCode, const TArray<uint8>& InKeyResponseData)
{
	// The response is a JSON Web Key (JWK) as per RFC-7518
	bool bSuccess = true;
	TArray<FClearKeyKIDKey> NewLicenseKeys;
	LastErrorMsg.Empty();
	if (HttpResponseCode == 200)
	{
#if ENABLE_LEGACY_RAWKEY_OVERRIDE
		if (InKeyResponseData.Num() == 16)
		{
			FScopeLock lock(&Lock);
			for(auto &KID : PendingRequiredKIDs)
			{
				FClearKeyKIDKey NewKey;
				if (Base64UrlDecode(NewKey.KID, KID))
				{
					TArray<IMediaCDM::FCDMCandidate> Configs(GetConfigurationsForKID(NewKey.KID));
					NewKey.EncryptionScheme = Configs.Num() ? GetCommonSchemeFromConfiguration(Configs[0]) : EClearKeyScheme::Unsupported;
					NewKey.Key = InKeyResponseData;
					NewLicenseKeys.Emplace(MoveTemp(NewKey));
				}
			}
		}
		else
#endif
		{
			// Is the JSON starting with an UTF-8 BOM?
			int32 ResponseOffset = InKeyResponseData.Num() > 3 && InKeyResponseData[0] == 0xEF && InKeyResponseData[1] == 0xBB && InKeyResponseData[2] == 0xBF ? 3 : 0;
			FString JsonString = ArrayToString(InKeyResponseData, ResponseOffset);
			TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(JsonString);
			TSharedPtr<FJsonObject> KeyResponse;
			if (FJsonSerializer::Deserialize(Reader, KeyResponse))
			{
				TArray<TSharedPtr<FJsonValue>> Keys;
				Keys = KeyResponse->GetArrayField(TEXT("keys"));
				for(int32 i=0; i<Keys.Num(); ++i)
				{
					auto Key = Keys[i]->AsObject();
					FString KTY = Key->GetStringField(TEXT("kty"));	// Mandatory (see RFC-7518 Section 6.1), must be "oct"
					if (KTY.Equals(TEXT("oct")))
					{
						FString KID = Key->GetStringField(TEXT("kid"));
						FString K = Key->GetStringField(TEXT("k"));
						if (KID.Len() && K.Len())
						{
							FClearKeyKIDKey NewKey;
							bool bOk = Base64UrlDecode(NewKey.KID, KID);
							if (bOk)
							{
								bOk = Base64UrlDecode(NewKey.Key, K);
							}
							if (bOk)
							{
								TArray<IMediaCDM::FCDMCandidate> Configs(GetConfigurationsForKID(NewKey.KID));
								// We require all KIDs to be unambiguously using the same encryption scheme
								// so it is sufficient to look at the first configuration.
								// If same KID configurations use different schemes this is an authoring error.
								NewKey.EncryptionScheme = Configs.Num() ? GetCommonSchemeFromConfiguration(Configs[0]) : EClearKeyScheme::Unsupported;
								NewLicenseKeys.Emplace(MoveTemp(NewKey));
							}
							else
							{
								LastErrorMsg = TEXT("Could not base64 decode k or kid in response");
								bSuccess = false;
								break;
							}
						}
					}
					else
					{
						LastErrorMsg = TEXT("Unexpected key type (kty) in response");
						bSuccess = false;
						break;
					}
				}
			}
			else
			{
				LastErrorMsg = TEXT("Failed to parse license response");
				bSuccess = false;
			}
		}

		if (bSuccess)
		{
			TSharedPtr<FClearKeyCDM, ESPMode::ThreadSafe> CDM = OwningCDM.Pin();
			if (CDM.IsValid())
			{
				CDM->AddPlayerSessionKeys(PlayerSession, NewLicenseKeys);
			}

			AddKeyKIDs(NewLicenseKeys);
			Lock.Lock();
			CurrentState = ECDMState::Ready;
			Lock.Unlock();

			UpdateKeyWithDecrypters();
			return ECDMError::Success;
		}
	}
	else
	{
		LastErrorMsg = FString::Printf(TEXT("Received bad license key response. HTTP code %d"), HttpResponseCode);
	}
	CurrentState = ECDMState::InvalidKey;
	UpdateStateWithDecrypters(CurrentState);
	return ECDMError::Failure;
}

//-----------------------------------------------------------------------------
/**
 * Creates a decrypter instance.
 * If the license keys have not been obtained yet the decrypter will not be
 * usable until the keys arrive.
 * One decrypter instance is created per elementary stream to decode.
 */
ECDMError FClearKeyDRMClient::CreateDecrypter(TSharedPtr<IMediaCDMDecrypter, ESPMode::ThreadSafe>& OutDecrypter, const FString& InMimeType)
{
	TSharedPtr<FClearKeyDRMDecrypter, ESPMode::ThreadSafe> NewDec = MakeShared<FClearKeyDRMDecrypter, ESPMode::ThreadSafe>();
	NewDec->Initialize(InMimeType);

	FScopeLock lock(&Lock);

	// The initial state of the decrypter is the same as the one of the client.
	NewDec->SetState(CurrentState);
	NewDec->SetLastErrorMessage(LastErrorMsg);
	// If ready set the key with the decrypter.
	if (CurrentState == ECDMState::Ready)
	{
		NewDec->SetLicenseKeys(LicenseKeys);
	}

	RemoveStaleDecrypters();
	Decrypters.Emplace(NewDec);
	OutDecrypter = NewDec;
	return ECDMError::Success;
}


//-----------------------------------------------------------------------------
/**
 * Removes decrypters that the application no longer uses.
 */
void FClearKeyDRMClient::RemoveStaleDecrypters()
{
	for(int32 i=0; i<Decrypters.Num(); ++i)
	{
		if (!Decrypters[i].IsValid())
		{
			Decrypters.RemoveAt(i);
			--i;
		}
	}
}

//-----------------------------------------------------------------------------
/**
 * Updates all this client's decrypters with the new set of license keys.
 */
void FClearKeyDRMClient::UpdateKeyWithDecrypters()
{
	FScopeLock lock(&Lock);
	RemoveStaleDecrypters();
	for(int32 i=0; i<Decrypters.Num(); ++i)
	{
		TSharedPtr<FClearKeyDRMDecrypter, ESPMode::ThreadSafe> Decrypter = Decrypters[i].Pin();
		Decrypter->SetLicenseKeys(LicenseKeys);
	}
}

//-----------------------------------------------------------------------------
/**
 * Sets the state of all this client's decrypters to the given state.
 * This is called in cases of license errors to make all instances fail.
 */
void FClearKeyDRMClient::UpdateStateWithDecrypters(ECDMState InNewState)
{
	FScopeLock lock(&Lock);
	RemoveStaleDecrypters();
	for(int32 i=0; i<Decrypters.Num(); ++i)
	{
		TSharedPtr<FClearKeyDRMDecrypter, ESPMode::ThreadSafe> Decrypter = Decrypters[i].Pin();
		Decrypter->SetLastErrorMessage(LastErrorMsg);
		Decrypter->SetState(InNewState);
	}
}

//-----------------------------------------------------------------------------
/**
 * Extracts relevant information from the AdditionalElements
 */
void FClearKeyDRMClient::GetValuesFromConfigurations()
{
	// For ClearKey we do not expect to get much additional elements.
	// While there could be a 'pssh' box it would not contain anything
	// besides the scheme id and the KIDs (in box version 1), which we
	// expect to have already been set up in the configuration.
	// What we need is the license server URL or URLs.
	FScopeLock lock(&Lock);
	for(auto &Config : CDMConfigurations)
	{
		if (Config.AdditionalElements.IsEmpty())
		{
			continue;
		}

		TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(Config.AdditionalElements);
		TSharedPtr<FJsonObject> ConfigJSON;
		if (FJsonSerializer::Deserialize(Reader, ConfigJSON))
		{
			// Try dashif:laurl first.
			if (GetURLsFrom(LicenseServerURLsFromConfigs, ConfigJSON, ClearKeyCDM::UrlAttrib_dashif_laurl, ClearKeyCDM::UrlAttrib_laurl))
			{
				continue;
			}
			// Then the deprecated clearkey:Laurl
			else if (GetURLsFrom(LicenseServerURLsFromConfigs, ConfigJSON, ClearKeyCDM::UrlAttrib_clearkey_laurl, ClearKeyCDM::UrlAttrib_Laurl))
			{
				continue;
			}
		}
		else
		{
			LastErrorMsg = TEXT("Could not parse additional configuration element.");
			CurrentState = ECDMState::ConfigurationError;
			return;
		}
	}
}

//-----------------------------------------------------------------------------
/**
 * Tries to get URLs from a JSON value, which could either be a string, an object
 * with additional properties, or an array of the same.
 */
bool FClearKeyDRMClient::GetURLsFrom(TArray<FString>& OutURLs, const TSharedPtr<FJsonValue>& JSONValue, const FString& PropertyName, const FString& AltPropertyName)
{
	// Try string first
	FString String;
	if (JSONValue->TryGetString(String))
	{
		OutURLs.AddUnique(MoveTemp(String));
		return true;
	}
	// Try object next. This is when the URL has additional properties.
	const TSharedPtr<FJsonObject>* Object = nullptr;
	if (JSONValue->TryGetObject(Object))
	{
		if ((*Object)->TryGetStringField(ClearKeyCDM::JSONTextPropertyName, String))
		{
			OutURLs.AddUnique(MoveTemp(String));
			return true;
		}
		// TBD: should we recurse into the object? For now let's not.
		//return GetURLsFrom(OutURLs, *Object, PropertyName, AltPropertyName);
	}
	// Finally try array
	const TArray<TSharedPtr<FJsonValue>>* Array = nullptr;
	if (JSONValue->TryGetArray(Array))
	{
		bool bAny = false;
		for(auto &Val : *Array)
		{
			if (GetURLsFrom(OutURLs, Val, PropertyName, AltPropertyName))
			{
				bAny = true;
			}
		}
		return bAny;
	}
	return false;
}

//-----------------------------------------------------------------------------
/**
 * Tries to get URLs from a JSON object, which could either be a string, an object
 * with additional properties, or an array of the same.
 */
bool FClearKeyDRMClient::GetURLsFrom(TArray<FString>& OutURLs, const TSharedPtr<FJsonObject>& JSON, const FString& PropertyName, const FString& AltPropertyName)
{
	TArray<FString> StringArray;
	FString String;
	// Try single URL without attributes first.
	if (JSON->TryGetStringField(PropertyName, String) || (AltPropertyName.Len() && JSON->TryGetStringField(AltPropertyName, String)))
	{
		OutURLs.AddUnique(MoveTemp(String));
		return true;
	}
	// Try as array of URLs without attributes.
	else if (JSON->TryGetStringArrayField(PropertyName, StringArray) || (AltPropertyName.Len() && JSON->TryGetStringArrayField(AltPropertyName, StringArray)))
	{
		for(auto &It : StringArray)
		{
			OutURLs.AddUnique(It);
		}
		return true;
	}
	// URL may have additional properties, turning it from a string or string array into an object or array of objects.
	const TSharedPtr<FJsonObject>* Object = nullptr;
	// Try single object. Attributes are of no interest at the moment.
	if (JSON->TryGetObjectField(PropertyName, Object) || (AltPropertyName.Len() && JSON->TryGetObjectField(AltPropertyName, Object)))
	{
		if ((*Object)->TryGetStringField(ClearKeyCDM::JSONTextPropertyName, String))
		{
			OutURLs.AddUnique(MoveTemp(String));
			return true;
		}
	}
	// Finally try as array of objects.
	const TArray<TSharedPtr<FJsonValue>>* ObjectArray = nullptr;
	if (JSON->TryGetArrayField(PropertyName, ObjectArray) || (AltPropertyName.Len() && JSON->TryGetArrayField(AltPropertyName, ObjectArray)))
	{
		bool bAny = false;
		for(auto &Val : *ObjectArray)
		{
			if (GetURLsFrom(OutURLs, Val, PropertyName, AltPropertyName))
			{
				bAny = true;
			}
		}
		return bAny;
	}
	return false;
}




//-----------------------------------------------------------------------------
/**
 * Creates a new decrypter instance.
 */
FClearKeyDRMDecrypter::FClearKeyDRMDecrypter()
{
}

//-----------------------------------------------------------------------------
/**
 * Destroys a decrypter instance.
 */
FClearKeyDRMDecrypter::~FClearKeyDRMDecrypter()
{
}

//-----------------------------------------------------------------------------
/**
 * Initializes a decrypter instance to default state.
 */
void FClearKeyDRMDecrypter::Initialize(const FString& InMimeType)
{
	FScopeLock lock(&Lock);
	CurrentState = ECDMState::Idle;
	KeyDecrypters.Empty();
}

//-----------------------------------------------------------------------------
/**
 * Updates the valid license keys with this decrypter instance.
 * All currently active keys are removed and replaced with the new ones.
 */
void FClearKeyDRMDecrypter::SetLicenseKeys(const TArray<FClearKeyKIDKey>& InLicenseKeys)
{
	FScopeLock lock(&Lock);
	LicenseKeys = InLicenseKeys;
	Reinitialize();
}

//-----------------------------------------------------------------------------
/**
 * Returns the current decrypter's state.
 */
ECDMState FClearKeyDRMDecrypter::GetState()
{
	FScopeLock lock(&Lock);
	// This is the amalgamated state of all the internal decrypters per
	// active key.
	return CurrentState;
}

//-----------------------------------------------------------------------------
/**
 * Returns the most recent error message.
 */
FString FClearKeyDRMDecrypter::GetLastErrorMessage()
{
	FScopeLock lock(&Lock);
	return LastErrorMsg;
}

//-----------------------------------------------------------------------------
/**
 * Sets a new state to this decrypter and all its currently active key decrypters.
 */
void FClearKeyDRMDecrypter::SetState(ECDMState InNewState)
{
	FScopeLock lock(&Lock);
	CurrentState = InNewState;
	for(int32 i=0; i<KeyDecrypters.Num(); ++i)
	{
		KeyDecrypters[i].State = InNewState;
	}
}

//-----------------------------------------------------------------------------
/**
 * Updates the last error message.
 */
void FClearKeyDRMDecrypter::SetLastErrorMessage(const FString& InNewErrorMessage)
{
	FScopeLock lock(&Lock);
	LastErrorMsg = InNewErrorMessage;
}


//-----------------------------------------------------------------------------
/**
 * Called by the application with PSSH box data to update the current set of
 * key IDs when key rotation is used.
 */
ECDMError FClearKeyDRMDecrypter::UpdateInitDataFromPSSH(const TArray<uint8>& InPSSHData)
{
	return ECDMError::NotSupported;
}

ECDMError FClearKeyDRMDecrypter::UpdateInitDataFromMultiplePSSH(const TArray<TArray<uint8>>& InPSSHData)
{
	return ECDMError::NotSupported;
}

//-----------------------------------------------------------------------------
/**
 * Update from a URL and additional scheme specific elements.
 */
ECDMError FClearKeyDRMDecrypter::UpdateFromURL(const FString& InURL, const FString& InAdditionalElements)
{
	return ECDMError::NotSupported;
}

//-----------------------------------------------------------------------------
/**
 * Locates the decrypter for the given key ID.
 */
FClearKeyDRMDecrypter::FKeyDecrypter* FClearKeyDRMDecrypter::GetDecrypterForKID(const TArray<uint8>& KID)
{
	// Note: The critical section must be locked already!
	for(int32 i=0; i<KeyDecrypters.Num(); ++i)
	{
		if (KID == KeyDecrypters[i].KIDKey.KID)
		{
			// Decrypter needs to be ready.
			if (KeyDecrypters[i].State == ECDMState::Ready)
			{
				return &KeyDecrypters[i];
			}
		}
	}
	return nullptr;
}

//-----------------------------------------------------------------------------
/**
 * Reinitializes the decrypter to its starting state.
 */
void FClearKeyDRMDecrypter::Reinitialize()
{
	FScopeLock lock(&Lock);

	KeyDecrypters.Empty();
	LastErrorMsg.Empty();

	CurrentState = ECDMState::Ready;
	for(int32 i=0; i<LicenseKeys.Num(); ++i)
	{
		FKeyDecrypter& kd = KeyDecrypters.AddDefaulted_GetRef();
		kd.KIDKey = LicenseKeys[i];
		kd.Decrypter = ElectraCDM::IStreamDecrypterAES128::Create();
		kd.State = ECDMState::Ready;
		kd.bIsInitialized = false;
	}
}

//-----------------------------------------------------------------------------
/**
 * Decrypts data in place according to the encrypted sample information.
 */
ECDMError FClearKeyDRMDecrypter::DecryptInPlace(uint8* InOutData, int32 InNumDataBytes, const FMediaCDMSampleInfo& InSampleInfo)
{
	FScopeLock lock(&Lock);
	LastErrorMsg.Empty();
	FClearKeyDRMDecrypter::FKeyDecrypter* DecrypterState = GetDecrypterForKID(InSampleInfo.DefaultKID);
	TSharedPtr<ElectraCDM::IStreamDecrypterAES128, ESPMode::ThreadSafe> Decrypter = DecrypterState ? DecrypterState->Decrypter : nullptr;
	if (Decrypter.IsValid() && DecrypterState)
	{
		ElectraCDM::IStreamDecrypterAES128::EResult Result;

		EClearKeyScheme SchemeFromMedia = FClearKeyCDM::GetCommonSchemeFromCipher(InSampleInfo.Scheme4CC);
		EClearKeyScheme SchemeFromKID = DecrypterState->KIDKey.EncryptionScheme;

		EClearKeyScheme SchemeToUse = SchemeFromMedia != EClearKeyScheme::Unsupported ? SchemeFromMedia : SchemeFromKID;

		// "cenc" scheme? (AES-128 CTR)
		if (SchemeToUse == EClearKeyScheme::Cenc)
		{
			if (!DecrypterState->bIsInitialized)
			{
				Result = Decrypter->CTRInit(DecrypterState->KIDKey.Key);
				if ((DecrypterState->bIsInitialized = Result == ElectraCDM::IStreamDecrypterAES128::EResult::Ok) == false)
				{
					DecrypterState->State = ECDMState::InvalidKey;
					CurrentState = ECDMState::InvalidKey;
					LastErrorMsg = TEXT("Invalid key");
					return ECDMError::Failure;
				}
			}

			if (Decrypter->CTRSetIV(InSampleInfo.IV) != ElectraCDM::IStreamDecrypterAES128::EResult::Ok)
			{
				LastErrorMsg = TEXT("Bad IV");
				return ECDMError::Failure;
			}
			if (InSampleInfo.SubSamples.Num() == 0)
			{
				Decrypter->CTRDecryptInPlace(InOutData, InNumDataBytes);
				return ECDMError::Success;
			}
			else
			{
				for(int32 i=0; i<InSampleInfo.SubSamples.Num(); ++i)
				{
					InOutData += InSampleInfo.SubSamples[i].NumClearBytes;
					if (InSampleInfo.SubSamples[i].NumEncryptedBytes)
					{
						Decrypter->CTRDecryptInPlace(InOutData, InSampleInfo.SubSamples[i].NumEncryptedBytes);
					}
					InOutData += InSampleInfo.SubSamples[i].NumEncryptedBytes;
				}
				return ECDMError::Success;
			}
		}
		// "cbcs" scheme? (AES-128 CBC)
		else if (SchemeToUse == EClearKeyScheme::Cbcs)
		{
			int32 NumBytesDecrypted = 0;

			auto DecryptPatternBlock = [&](int32 BlocksToGo) -> void
			{
				while(BlocksToGo > 0)
				{
					int32 NumEnc = BlocksToGo >= InSampleInfo.Pattern.CryptByteBlock ? InSampleInfo.Pattern.CryptByteBlock : BlocksToGo;
					/*Result =*/ Decrypter->CBCDecryptInPlace(NumBytesDecrypted, InOutData, NumEnc * 16, false);
					InOutData += (NumEnc + InSampleInfo.Pattern.SkipByteBlock) * 16;
					BlocksToGo -= InSampleInfo.Pattern.CryptByteBlock + InSampleInfo.Pattern.SkipByteBlock;
				}
			};

			if (InSampleInfo.SubSamples.Num() == 0)
			{
				Result = DecrypterState->Decrypter->CBCInit(DecrypterState->KIDKey.Key, &InSampleInfo.IV);
				if (Result != ElectraCDM::IStreamDecrypterAES128::EResult::Ok)
				{
					LastErrorMsg = TEXT("Bad key or IV length");
					return ECDMError::Failure;
				}

				// Entire sample is encrypted.
				if (InSampleInfo.Pattern.CryptByteBlock == 0 && InSampleInfo.Pattern.SkipByteBlock == 0)
				{
					/*Result =*/ Decrypter->CBCDecryptInPlace(NumBytesDecrypted, InOutData, InNumDataBytes & ~15, false);
				}
				else
				{
					DecryptPatternBlock(InNumDataBytes / 16);
				}
				return ECDMError::Success;
			}
			else
			{
				for(int32 i=0; i<InSampleInfo.SubSamples.Num(); ++i)
				{
					InOutData += InSampleInfo.SubSamples[i].NumClearBytes;

					if (InSampleInfo.SubSamples[i].NumEncryptedBytes)
					{
						// cbcs encryption is restarted with every sub-sample.
						if (InSampleInfo.IV.Num() == 16)
						{
							Result = DecrypterState->Decrypter->CBCInit(DecrypterState->KIDKey.Key, &InSampleInfo.IV);
						}
						else if (InSampleInfo.IV.Num() < 16)
						{
							TArray<uint8> paddedIV(InSampleInfo.IV);
							paddedIV.InsertZeroed(paddedIV.Num(), 16-paddedIV.Num());
							Result = DecrypterState->Decrypter->CBCInit(DecrypterState->KIDKey.Key, &paddedIV);
						}
						else
						{
							Result = ElectraCDM::IStreamDecrypterAES128::EResult::BadIVLength;
						}
						if (Result != ElectraCDM::IStreamDecrypterAES128::EResult::Ok)
						{
							LastErrorMsg = TEXT("Bad key or IV length");
							return ECDMError::Failure;
						}

						if (InSampleInfo.Pattern.CryptByteBlock == 0 && InSampleInfo.Pattern.SkipByteBlock == 0)
						{
							/*Result =*/ Decrypter->CBCDecryptInPlace(NumBytesDecrypted, InOutData, InSampleInfo.SubSamples[i].NumEncryptedBytes & ~15, false);
							InOutData = InOutData + InSampleInfo.SubSamples[i].NumEncryptedBytes;
						}
						else
						{
							uint8* EncDataStart = InOutData;
							DecryptPatternBlock((int32) InSampleInfo.SubSamples[i].NumEncryptedBytes / 16);
							// The number of encrypted bytes in the subsample is not necessarily a 16B multiple.
							// If not, the last few bytes are not encrypted and as such are not touched by
							// decryption. We need to advance the pointer to the data to the actual number of
							// bytes - NOT the number of bytes that were decrypted! - to get to the next subsample.
							InOutData = EncDataStart + InSampleInfo.SubSamples[i].NumEncryptedBytes;
						}
					}
				}
				return ECDMError::Success;
			}
		}
		else
		{
			LastErrorMsg = TEXT("Unsupported encryption scheme for KID");
			return ECDMError::Failure;
		}
	}
	LastErrorMsg = TEXT("No valid decrypter found for KID");
	return ECDMError::Failure;
}


bool FClearKeyDRMDecrypter::IsBlockStreamDecrypter()
{
	return false;
}

ECDMError FClearKeyDRMDecrypter::BlockStreamDecryptStart(IStreamDecryptHandle*& OutStreamDecryptContext, const FMediaCDMSampleInfo& InSampleInfo)
{
	FScopeLock lock(&Lock);
	OutStreamDecryptContext = nullptr;
	LastErrorMsg = TEXT("Not a block stream decrypter");
	return ECDMError::CipherModeMismatch;
}
ECDMError FClearKeyDRMDecrypter::BlockStreamDecryptInPlace(IStreamDecryptHandle* InOutStreamDecryptContext, int32& OutNumBytesDecrypted, uint8* InOutData, int32 InNumDataBytes, bool bIsLastBlock)
{
	FScopeLock lock(&Lock);
	LastErrorMsg = TEXT("Not a block stream decrypter");
	return ECDMError::CipherModeMismatch;
}
ECDMError FClearKeyDRMDecrypter::BlockStreamDecryptEnd(IStreamDecryptHandle* InStreamDecryptContext)
{
	FScopeLock lock(&Lock);
	LastErrorMsg = TEXT("Not a block stream decrypter");
	return ECDMError::CipherModeMismatch;
}


}
