// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ElectraCDM.h"
#include "ElectraCDMSystem.h"

namespace ElectraCDM
{

class IClearKeyCDM : public IMediaCDMSystem
{
public:
	static void RegisterWith(IMediaCDM& InDRMManager);
	virtual ~IClearKeyCDM() = default;
	virtual FString GetLastErrorMessage() = 0;
	virtual const TArray<FString>& GetSchemeIDs() = 0;

	/*
		Elements that are parsed from a XML document, like:
			<ContentProtection schemeIdUri="urn:uuid:e2719d58-a985-b3c9-781a-b030af78d30e" value="ClearKey1.0">
			  <clearkey:Laurl Lic_type="EME-1.0">https://clearkey.example.com/AcquireLicense</clearkey:Laurl>
			  <dashif:laurl>https://clearkey2.example.com/AcquireLicense</dashif:laurl>
			  <cenc:pssh>YmFzZTY0IGVuY29kZWQgY29udGVudHMgb2YgkXBzc2iSIGJveCB3aXRoIHRoaXMgU3lzdGVtSUQ=</cenc:pssh>
			<ContentProtection>

		have these elements converted into JSON like:

			{
				"Laurl": {
					"Lic_type": "EME-1.0",
					"#text": "https://clearkey.example.com/AcquireLicense"
				},
				"laurl": "https://clearkey2.example.com/AcquireLicense",
				"pssh": "YmFzZTY0IGVuY29kZWQgY29udGVudHMgb2YgkXBzc2iSIGJveCB3aXRoIHRoaXMgU3lzdGVtSUQ="
			}

		where the text of the XML element is put into a property named "#text". XML namespaces are removed in this example
		(bOutNoNamespaces returned 'true') and no attribute prefix was set.
	*/
	virtual void GetCDMCustomJSONPrefixes(FString& OutAttributePrefix, FString& OutTextPropertyName, bool& bOutNoNamespaces) = 0;

	virtual TSharedPtr<IMediaCDMCapabilities, ESPMode::ThreadSafe> GetCDMCapabilities(const FString& InValue, const FString& InAdditionalElements) = 0;
	virtual ECDMError CreateDRMClient(TSharedPtr<IMediaCDMClient, ESPMode::ThreadSafe>& OutClient, IMediaCDM::IPlayerSession* InForPlayerSession, const TArray<IMediaCDM::FCDMCandidate>& InCandidates) = 0;
	virtual ECDMError ReleasePlayerSessionKeys(IMediaCDM::IPlayerSession* PlayerSession) = 0;
};

}

