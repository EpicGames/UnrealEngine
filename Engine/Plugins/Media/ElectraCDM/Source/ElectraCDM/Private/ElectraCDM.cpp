// Copyright Epic Games, Inc. All Rights Reserved.

#include "ElectraCDM.h"
#include "ElectraCDMSystem.h"
#include "HAL/CriticalSection.h"
#include "Misc/ScopeLock.h"

namespace ElectraCDM
{

struct IMediaCDM::IPlayerSession
{
};

class FMediaDRM : public IMediaCDM
{
public:
	static FMediaDRM& Get()
	{
		static FMediaDRM This;
		return This;
	}

	FMediaDRM();
	virtual ~FMediaDRM();

	virtual void RegisterCDM(TWeakPtr<IMediaCDMSystem, ESPMode::ThreadSafe> InCDMSystem) override;
	virtual void GetCDMCustomJSONPrefixes(const FString& InCDMScheme, const FString& InValue, FString& OutAttributePrefix, FString& OutTextPropertyName, bool& bOutNoNamespaces) override;
	virtual FString GetLastErrorMessage() override;
	virtual TSharedPtr<IMediaCDMCapabilities, ESPMode::ThreadSafe> GetCDMCapabilitiesForScheme(const FString& InCDMScheme, const FString& InValue, const FString& InAdditionalElements) override;

	virtual IPlayerSession* CreatePlayerSessionID() override
	{ return new IPlayerSession; }
	virtual void ReleasePlayerSessionID(IPlayerSession* PlayerSession) override
	{ delete PlayerSession; }

	virtual ECDMError CreateDRMClient(TSharedPtr<IMediaCDMClient, ESPMode::ThreadSafe>& OutClient, IPlayerSession* InForPlayerSession, const TArray<FCDMCandidate>& InCandidates) override;
	virtual ECDMError ReleasePlayerSessionKeys(IPlayerSession* PlayerSession) override;
private:
	void RemoveUnusedSystems();

	bool CompareSchemeIds(const FString& schemeA, const FString& schemeB)
	{
		if (schemeA.Equals(schemeB, ESearchCase::IgnoreCase))
		{
			return true;
		}
		FString sa(schemeA);
		FString sb(schemeB);
		if (sa.StartsWith(TEXT("urn:uuid:")))
		{
			sa.RightChopInline(9);
		}
		if (sb.StartsWith(TEXT("urn:uuid:")))
		{
			sb.RightChopInline(9);
		}
		sa.ReplaceInline(TEXT("-"), TEXT(""), ESearchCase::CaseSensitive);
		sb.ReplaceInline(TEXT("-"), TEXT(""), ESearchCase::CaseSensitive);
		return sa.Equals(sb, ESearchCase::IgnoreCase);
	}
	bool CompareSchemeIds(const FString& schemeA, const TArray<FString>& listOfSchemesB)
	{
		for(auto& schemeB : listOfSchemesB)
		{
			if (CompareSchemeIds(schemeA, schemeB))
			{
				return true;
			}
		}
		return false;
	}

	FCriticalSection Lock;
	TArray<TWeakPtr<IMediaCDMSystem, ESPMode::ThreadSafe>> CDMSystems;
	FString LastErrorMessage;
};

FMediaDRM::FMediaDRM()
{
}

FMediaDRM::~FMediaDRM()
{
}

void FMediaDRM::RemoveUnusedSystems()
{
	FScopeLock lock(&Lock);
	for(int32 i=0; i<CDMSystems.Num(); ++i)
	{
		if (!CDMSystems[i].IsValid())
		{
			CDMSystems.RemoveAt(i);
			--i;
		}
	}
}


void FMediaDRM::RegisterCDM(TWeakPtr<IMediaCDMSystem, ESPMode::ThreadSafe> InCDMSystem)
{
	RemoveUnusedSystems();
	FScopeLock lock(&Lock);
	CDMSystems.Emplace(InCDMSystem);
}


void FMediaDRM::GetCDMCustomJSONPrefixes(const FString& InCDMScheme, const FString& InValue, FString& OutAttributePrefix, FString& OutTextPropertyName, bool& bOutNoNamespaces)
{
	OutAttributePrefix.Empty();
	OutTextPropertyName.Empty();
	bOutNoNamespaces = false;
	RemoveUnusedSystems();
	TSharedPtr<IMediaCDMCapabilities, ESPMode::ThreadSafe> Capabilities;
	FScopeLock lock(&Lock);
	for(int32 i=0; i<CDMSystems.Num(); ++i)
	{
		TSharedPtr<IMediaCDMSystem, ESPMode::ThreadSafe> CDMSystem = CDMSystems[i].Pin();
		if (CDMSystem.IsValid() && CompareSchemeIds(InCDMScheme, CDMSystem->GetSchemeIDs()))
		{
			CDMSystem->GetCDMCustomJSONPrefixes(OutAttributePrefix, OutTextPropertyName, bOutNoNamespaces);
			break;
		}
	}
}


TSharedPtr<IMediaCDMCapabilities, ESPMode::ThreadSafe> FMediaDRM::GetCDMCapabilitiesForScheme(const FString& InCDMScheme, const FString& InValue, const FString& InAdditionalElements)
{
	RemoveUnusedSystems();
	TSharedPtr<IMediaCDMCapabilities, ESPMode::ThreadSafe> Capabilities;
	FScopeLock lock(&Lock);
	for(int32 i=0; i<CDMSystems.Num(); ++i)
	{
		TSharedPtr<IMediaCDMSystem, ESPMode::ThreadSafe> CDMSystem = CDMSystems[i].Pin();
		if (CDMSystem.IsValid() && CompareSchemeIds(InCDMScheme, CDMSystem->GetSchemeIDs()))
		{
			Capabilities = CDMSystem->GetCDMCapabilities(InValue, InAdditionalElements);
			if (Capabilities.IsValid())
			{
				break;
			}
		}
	}
	return Capabilities;
}


FString FMediaDRM::GetLastErrorMessage()
{
	FScopeLock lock(&Lock);
	return LastErrorMessage;
}



ECDMError FMediaDRM::CreateDRMClient(TSharedPtr<IMediaCDMClient, ESPMode::ThreadSafe>& OutClient, IPlayerSession* InForPlayerSession, const TArray<FCDMCandidate>& InCandidates)
{
	RemoveUnusedSystems();

	LastErrorMessage.Empty();

	// For now we narrow the candidates down to one based on the schemeID only.
	if (InCandidates.Num() == 0)
	{
		LastErrorMessage = TEXT("No candidates passed");
		return ECDMError::Failure;
	}
	// And really, right now we pick whichever is the first one...
	TSharedPtr<IMediaCDMSystem, ESPMode::ThreadSafe> SelectedCDMSystem;
	FScopeLock lock(&Lock);
	for(int32 i=0; i<CDMSystems.Num(); ++i)
	{
		TSharedPtr<IMediaCDMSystem, ESPMode::ThreadSafe> CDMSystem = CDMSystems[i].Pin();
		if (CDMSystem.IsValid() && CompareSchemeIds(InCandidates[0].SchemeId, CDMSystem->GetSchemeIDs()))
		{
			SelectedCDMSystem = CDMSystem;
			break;
		}
	}

	// Add all matching candidates in. This is needed for different KIDs.
	if (SelectedCDMSystem.IsValid())
	{
		TArray<FCDMCandidate> CDMCandidateList;
		for(auto &Cand : InCandidates)
		{
			if (CompareSchemeIds(Cand.SchemeId, SelectedCDMSystem->GetSchemeIDs()))
			{
				CDMCandidateList.Emplace(Cand);
			}
		}
		ECDMError Error = SelectedCDMSystem->CreateDRMClient(OutClient, InForPlayerSession, CDMCandidateList);
		LastErrorMessage = SelectedCDMSystem->GetLastErrorMessage();
		return Error;
	}

	LastErrorMessage = TEXT("No matching CDM system registered");
	return ECDMError::Failure;
}

ECDMError FMediaDRM::ReleasePlayerSessionKeys(IPlayerSession* PlayerSession)
{
	RemoveUnusedSystems();
	FScopeLock lock(&Lock);
	for(int32 i=0; i<CDMSystems.Num(); ++i)
	{
		TSharedPtr<IMediaCDMSystem, ESPMode::ThreadSafe> CDMSystem = CDMSystems[i].Pin();
		CDMSystem->ReleasePlayerSessionKeys(PlayerSession);
	}
	return ECDMError::Success;
}




IMediaCDM& IMediaCDM::Get()
{
	return FMediaDRM::Get();
}



}

