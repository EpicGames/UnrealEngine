// Copyright Epic Games, Inc. All Rights Reserved.

#include "ElectraCDMModule.h"

#include "Modules/ModuleManager.h"
#include "IElectraCDMModule.h"

#include "ElectraCDM.h"
#include "ClearKey/ClearKeyCDM.h"
#include "HLS_AES128/HLS_AES128CDM.h"


#define LOCTEXT_NAMESPACE "ElectraCDMModule"

DEFINE_LOG_CATEGORY(LogElectraCDM);

// -----------------------------------------------------------------------------------------------------------------------------------

class FElectraCDMModule: public IElectraCDMModule
{
public:
	// IModuleInterface interface

	void StartupModule() override
	{
		ElectraCDM::IMediaCDM& Singleton = ElectraCDM::IMediaCDM::Get();

		// Register default CDMs
		ElectraCDM::IClearKeyCDM::RegisterWith(Singleton);
		ElectraCDM::IHLS_AES128_CDM::RegisterWith(Singleton);
	}

	void ShutdownModule() override
	{
	}

private:
};

IMPLEMENT_MODULE(FElectraCDMModule, ElectraCDM);

#undef LOCTEXT_NAMESPACE
