// Copyright Epic Games, Inc. All Rights Reserved.

#include "ElectraCDMUtils.h"
#include "Misc/Base64.h"

namespace ElectraCDMUtils
{
	FString Base64UrlEncode(const TArray<uint8>& InData)
	{
		FString b64 = FBase64::Encode(InData);
		// Base64Url encoding replaces '+' and '/' with '-' and '_' respectively.
		b64.ReplaceCharInline(TCHAR('+'), TCHAR('-'), ESearchCase::IgnoreCase);
		b64.ReplaceCharInline(TCHAR('/'), TCHAR('_'), ESearchCase::IgnoreCase);
		return b64;
	}
	bool Base64UrlDecode(TArray<uint8>& OutData, FString InString)
	{
		InString.ReplaceCharInline(TCHAR('-'), TCHAR('+'), ESearchCase::IgnoreCase);
		InString.ReplaceCharInline(TCHAR('_'), TCHAR('/'), ESearchCase::IgnoreCase);
		return FBase64::Decode(InString, OutData);
	}

	FString StripDashesFromKID(const FString& InKID)
	{
		return InKID.Replace(TEXT("-"), TEXT(""), ESearchCase::CaseSensitive);
	}

	void ConvertKIDToBin(TArray<uint8>& OutBinKID, const FString& InKID)
	{
		OutBinKID.Empty();
		check((InKID.Len() % 2) == 0);
		if ((InKID.Len() % 2) == 0)
		{
			OutBinKID.AddUninitialized(InKID.Len() / 2);
			HexToBytes(InKID, OutBinKID.GetData());
		}
	}

	FString ConvertKIDToBase64(const FString& InKID)
	{
		TArray<uint8> BinKID;
		ConvertKIDToBin(BinKID, InKID);
		FString b64 = Base64UrlEncode(BinKID);
		// Chop off trailing padding.
		return b64.Replace(TEXT("="), TEXT(""), ESearchCase::CaseSensitive);
	}

	void StringToArray(TArray<uint8>& OutArray, const FString& InString)
	{
		FTCHARToUTF8 cnv(*InString);
		int32 Len = cnv.Length();
		OutArray.AddUninitialized(Len);
		FMemory::Memcpy(OutArray.GetData(), cnv.Get(), Len);
	}

	FString ArrayToString(const TArray<uint8>& InArray, int32 InStartAt)
	{
		FUTF8ToTCHAR cnv((const ANSICHAR*)InArray.GetData() + InStartAt, InArray.Num() - InStartAt);
		FString UTF8Text = FString::ConstructFromPtrSize(cnv.Get(), cnv.Length());
		return MoveTemp(UTF8Text);
	}

}
