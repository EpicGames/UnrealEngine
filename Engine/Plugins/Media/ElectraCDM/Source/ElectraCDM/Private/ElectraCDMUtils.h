// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <CoreMinimal.h>

namespace ElectraCDMUtils
{
	FString Base64UrlEncode(const TArray<uint8>& InData);
	bool Base64UrlDecode(TArray<uint8>& OutData, FString InString);
	FString StripDashesFromKID(const FString& InKID);
	void ConvertKIDToBin(TArray<uint8>& OutBinKID, const FString& InKID);
	FString ConvertKIDToBase64(const FString& InKID);
	void StringToArray(TArray<uint8>& OutArray, const FString& InString);
	FString ArrayToString(const TArray<uint8>& InArray, int32 InStartAt=0);
}
