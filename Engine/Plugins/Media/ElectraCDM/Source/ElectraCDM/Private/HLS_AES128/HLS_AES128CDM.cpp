// Copyright Epic Games, Inc. All Rights Reserved.

#include "HLS_AES128/HLS_AES128CDM.h"
#include "ElectraCDM.h"
#include "ElectraCDMClient.h"
#include "Crypto/StreamCryptoAES128.h"
#include "ElectraCDMUtils.h"
#include "ElectraCDMModule.h"
#include <Misc/Base64.h>
#include <Misc/ScopeLock.h>
#include <Dom/JsonObject.h>
#include <Serialization/JsonReader.h>
#include <Serialization/JsonSerializer.h>


namespace ElectraCDM
{

namespace HLSKeyParamNames
{
static FString URI(TEXT("URI"));
}


enum class EHLSEncryptionScheme
{
	Unsupported,
	Cenc,
	Cbcs,
	// AES128, CBC with PKCS7 padding
	Cbc7
};

struct FHLSKIDKey
{
	TArray<uint8> KID;
	TArray<uint8> Key;
	EHLSEncryptionScheme EncryptionScheme = EHLSEncryptionScheme::Unsupported;
};


class FHLS_AES128_CDM : public IHLS_AES128_CDM, public IMediaCDMCapabilities, public TSharedFromThis<FHLS_AES128_CDM, ESPMode::ThreadSafe>
{
public:
	FHLS_AES128_CDM() = default;
	virtual ~FHLS_AES128_CDM() = default;
	virtual FString GetLastErrorMessage() override;
	virtual const TArray<FString>& GetSchemeIDs() override;
	virtual void GetCDMCustomJSONPrefixes(FString& OutAttributePrefix, FString& OutTextPropertyName, bool& bOutNoNamespaces) override;
	virtual TSharedPtr<IMediaCDMCapabilities, ESPMode::ThreadSafe> GetCDMCapabilities(const FString& InValue, const FString& InAdditionalElements) override;
	virtual ECDMError CreateDRMClient(TSharedPtr<IMediaCDMClient, ESPMode::ThreadSafe>& OutClient, IMediaCDM::IPlayerSession* InForPlayerSession, const TArray<IMediaCDM::FCDMCandidate>& InCandidates) override;
	virtual ECDMError ReleasePlayerSessionKeys(IMediaCDM::IPlayerSession* PlayerSession) override;

	virtual ESupportResult SupportsCipher(const FString& InCipherType) override;
	virtual ESupportResult SupportsType(const FString& InMimeType) override;
	virtual ESupportResult RequiresSecureDecoder(const FString& InMimeType) override;

	void AddPlayerSessionKeys(IMediaCDM::IPlayerSession* InPlayerSession, const TArray<FHLSKIDKey>& InNewSessionKeys);
	bool GetPlayerSessionKey(FHLSKIDKey& OutKey, IMediaCDM::IPlayerSession* InPlayerSession, const TArray<uint8>& InForKID);

	static EHLSEncryptionScheme GetCommonSchemeFromCipher(const FString& InCipherName);
	static EHLSEncryptionScheme GetCommonSchemeFromCipher(uint32 InCipher4CC);
public:
	static TSharedPtr<FHLS_AES128_CDM, ESPMode::ThreadSafe> Get();

	FCriticalSection Lock;
	TMap<IMediaCDM::IPlayerSession*, TArray<FHLSKIDKey>> ActiveLicensesPerPlayer;
	FString LastErrorMessage;
};


class FHLSDRMDecrypter : public IMediaCDMDecrypter
{
public:
	FHLSDRMDecrypter();
	void Initialize(const FString& InMimeType);
	void SetLicenseKeys(const TArray<FHLSKIDKey>& InLicenseKeys);
	void SetState(ECDMState InNewState);
	void SetLastErrorMessage(const FString& InNewErrorMessage);
	virtual ~FHLSDRMDecrypter();
	virtual ECDMState GetState() override;
	virtual FString GetLastErrorMessage() override;
	virtual ECDMError UpdateInitDataFromPSSH(const TArray<uint8>& InPSSHData) override;
	virtual ECDMError UpdateInitDataFromMultiplePSSH(const TArray<TArray<uint8>>& InPSSHData) override;
	virtual ECDMError UpdateFromURL(const FString& InURL, const FString& InAdditionalElements) override;
	virtual bool IsBlockStreamDecrypter() override;
	virtual void Reinitialize() override;
	virtual ECDMError DecryptInPlace(uint8* InOutData, int32 InNumDataBytes, const FMediaCDMSampleInfo& InSampleInfo) override;
	virtual ECDMError BlockStreamDecryptStart(IStreamDecryptHandle*& OutStreamDecryptContext, const FMediaCDMSampleInfo& InSampleInfo) override;
	virtual ECDMError BlockStreamDecryptInPlace(IStreamDecryptHandle* InOutStreamDecryptContext, int32& OutNumBytesDecrypted, uint8* InOutData, int32 InNumDataBytes, bool bIsLastBlock) override;
	virtual ECDMError BlockStreamDecryptEnd(IStreamDecryptHandle* InStreamDecryptContext) override;

private:
	struct FBlockDecrypterHandle : public IStreamDecryptHandle
	{
		TArray<uint8> KID;
	};

	struct FKeyDecrypter
	{
		FHLSKIDKey KIDKey;
		TSharedPtr<ElectraCDM::IStreamDecrypterAES128, ESPMode::ThreadSafe> Decrypter;
		ECDMState State = ECDMState::Idle;
		bool bIsInitialized = false;
	};

	FHLSDRMDecrypter::FKeyDecrypter* GetDecrypterForKID(const TArray<uint8>& KID);

	FCriticalSection Lock;
	TArray<FHLSKIDKey> LicenseKeys;
	TArray<FKeyDecrypter> KeyDecrypters;
	ECDMState CurrentState = ECDMState::Idle;
	FString MimeType;
	FString LastErrorMsg;
};


class FHLSDRMClient : public IMediaCDMClient, public TSharedFromThis<FHLSDRMClient, ESPMode::ThreadSafe>
{
public:
	virtual ~FHLSDRMClient();

	FHLSDRMClient();
	void Initialize(TSharedPtr<FHLS_AES128_CDM, ESPMode::ThreadSafe> InOwningCDM, IMediaCDM::IPlayerSession* InForPlayerSession, const TArray<IMediaCDM::FCDMCandidate>& InCDMConfigurations);

	virtual ECDMState GetState() override;
	virtual FString GetLastErrorMessage() override;

	virtual void RegisterEventListener(TWeakPtr<IMediaCDMEventListener, ESPMode::ThreadSafe> InEventListener) override;
	virtual void UnregisterEventListener(TWeakPtr<IMediaCDMEventListener, ESPMode::ThreadSafe> InEventListener) override;
	virtual void PrepareLicenses() override;
	virtual void SetLicenseServerURL(const FString& InLicenseServerURL) override;
	virtual void GetLicenseKeyURL(FString& OutLicenseURL) override;
	virtual void GetLicenseKeyRequestData(TArray<uint8>& OutKeyRequestData, FString& OutHttpMethod, TArray<FString>& OutHttpHeaders, uint32& OutFlags) override;
	virtual ECDMError SetLicenseKeyResponseData(void* InEventId, int32 HttpResponseCode, const TArray<uint8>& InKeyResponseData) override;
	virtual ECDMError CreateDecrypter(TSharedPtr<IMediaCDMDecrypter, ESPMode::ThreadSafe>& OutDecrypter, const FString& InMimeType) override;

private:
	EHLSEncryptionScheme GetCommonSchemeFromConfiguration(const IMediaCDM::FCDMCandidate& InConfiguration);
	TArray<IMediaCDM::FCDMCandidate> GetConfigurationsForKID(const TArray<uint8>& InForKID);
	int32 PrepareKIDsToRequest();
	void AddKeyKID(const FHLSKIDKey& InKeyKid);
	void AddKeyKIDs(const TArray<FHLSKIDKey>& InKeyKids);
	void FireEvent(IMediaCDMEventListener::ECDMEventType InEvent);
	void RemoveStaleDecrypters();
	void UpdateKeyWithDecrypters();
	void UpdateStateWithDecrypters(ECDMState InNewState);
	void GetValuesFromConfigurations();

	FCriticalSection Lock;
	IMediaCDM::IPlayerSession* PlayerSession = nullptr;
	TWeakPtr<FHLS_AES128_CDM, ESPMode::ThreadSafe> OwningCDM;
	TArray<IMediaCDM::FCDMCandidate> CDMConfigurations;
	TArray<TWeakPtr<IMediaCDMEventListener, ESPMode::ThreadSafe>> Listeners;
	TArray<TWeakPtr<FHLSDRMDecrypter, ESPMode::ThreadSafe>> Decrypters;
	TArray<FString> PendingRequiredKIDs;
	TArray<FHLSKIDKey> LicenseKeys;
	TOptional<FString> LicenseServerURLOverride;
	TArray<FString> LicenseServerURLsFromConfigs;
	ECDMState CurrentState = ECDMState::Idle;
	FString LastErrorMsg;
};

/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/


//-----------------------------------------------------------------------------
/**
 * Registers this CDM with the CDM manager
 */
void IHLS_AES128_CDM::RegisterWith(IMediaCDM& InDRMManager)
{
	InDRMManager.RegisterCDM(FHLS_AES128_CDM::Get());
}

//-----------------------------------------------------------------------------
/**
 * Returns the singleton of this CDM system.
 */
TSharedPtr<FHLS_AES128_CDM, ESPMode::ThreadSafe> FHLS_AES128_CDM::Get()
{
	static TSharedPtr<FHLS_AES128_CDM, ESPMode::ThreadSafe> This = MakeShared<FHLS_AES128_CDM, ESPMode::ThreadSafe>();
	return This;
}

//-----------------------------------------------------------------------------
/**
 * Returns an internal scheme ID for this CDM.
 * There is no actual scheme UUID so we use the `METHOD` as specified in the HLS RFC8216
 */
const TArray<FString>& FHLS_AES128_CDM::GetSchemeIDs()
{
	static TArray<FString> SchemeIDs({TEXT("AES-128"), TEXT("SAMPLE-AES"), TEXT("SAMPLE-AES-CTR") });
	return SchemeIDs;
}

//-----------------------------------------------------------------------------
/**
 * Returns the most recent error message.
 */
FString FHLS_AES128_CDM::GetLastErrorMessage()
{
	FScopeLock lock(&Lock);
	return LastErrorMessage;
}

//-----------------------------------------------------------------------------
/**
 * Returns the expected element prefixes for the AdditionalElements JSON.
 */
void FHLS_AES128_CDM::GetCDMCustomJSONPrefixes(FString& OutAttributePrefix, FString& OutTextPropertyName, bool& bOutNoNamespaces)
{
	// None used.
	OutAttributePrefix.Empty();
	OutTextPropertyName.Empty();
	bOutNoNamespaces = false;
}

//-----------------------------------------------------------------------------
/**
 * Returns the capability interface of this CDM.
 */
TSharedPtr<IMediaCDMCapabilities, ESPMode::ThreadSafe> FHLS_AES128_CDM::GetCDMCapabilities(const FString& InValue, const FString& InAdditionalElements)
{
	if (InValue.IsEmpty() || InValue.Equals(TEXT("identity")))
	{
		return AsShared();
	}
	return nullptr;
}

//-----------------------------------------------------------------------------
/**
 * Creates a client instance of this CDM.
 * The application may create one or more instances, possibly for different key IDs.
 */
ECDMError FHLS_AES128_CDM::CreateDRMClient(TSharedPtr<IMediaCDMClient, ESPMode::ThreadSafe>& OutClient, IMediaCDM::IPlayerSession* InForPlayerSession, const TArray<IMediaCDM::FCDMCandidate>& InCandidates)
{
	FHLSDRMClient* NewClient = new FHLSDRMClient;
	NewClient->Initialize(AsShared(), InForPlayerSession, InCandidates);
	OutClient = MakeShareable(NewClient);
	FScopeLock lock(&Lock);
	LastErrorMessage.Empty();
	return ECDMError::Success;
}

//-----------------------------------------------------------------------------
/**
 * Adds keys to the specified player session.
 */
void FHLS_AES128_CDM::AddPlayerSessionKeys(IMediaCDM::IPlayerSession* InPlayerSession, const TArray<FHLSKIDKey>& InNewSessionKeys)
{
	FScopeLock lock(&Lock);
	TArray<FHLSKIDKey>& Keys = ActiveLicensesPerPlayer.FindOrAdd(InPlayerSession);
	for(auto &NewKey : InNewSessionKeys)
	{
		bool bHaveAlready = false;
		for(auto &HaveKey : Keys)
		{
			if (HaveKey.KID == NewKey.KID)
			{
				bHaveAlready = true;
				break;
			}
		}
		if (!bHaveAlready)
		{
			Keys.Emplace(NewKey);
		}
	}
}

//-----------------------------------------------------------------------------
/**
 * Returns a player session's key for the specified KID.
 */
bool FHLS_AES128_CDM::GetPlayerSessionKey(FHLSKIDKey& OutKey, IMediaCDM::IPlayerSession* InPlayerSession, const TArray<uint8>& InForKID)
{
	FScopeLock lock(&Lock);
	const TArray<FHLSKIDKey>* Keys = ActiveLicensesPerPlayer.Find(InPlayerSession);
	if (Keys)
	{
		for(auto &Key : *Keys)
		{
			if (Key.KID == InForKID)
			{
				OutKey = Key;
				return true;
			}
		}
	}
	return false;
}

//-----------------------------------------------------------------------------
/**
 * Releases all keys the specified player session has acquired.
 */
ECDMError FHLS_AES128_CDM::ReleasePlayerSessionKeys(IMediaCDM::IPlayerSession* PlayerSession)
{
	FScopeLock lock(&Lock);
	LastErrorMessage.Empty();
	ActiveLicensesPerPlayer.Remove(PlayerSession);
	return ECDMError::Success;
}


//-----------------------------------------------------------------------------
/**
 * Returns if a specified cipher (eg. "cenc" or "cbcs") is supported by this CDM.
 */
IMediaCDMCapabilities::ESupportResult FHLS_AES128_CDM::SupportsCipher(const FString& InCipherType)
{
	return GetCommonSchemeFromCipher(InCipherType) != EHLSEncryptionScheme::Unsupported ? IMediaCDMCapabilities::ESupportResult::Supported : IMediaCDMCapabilities::ESupportResult::NotSupported;
}

//-----------------------------------------------------------------------------
/**
 * Returns if a media stream of a given format can be decrypted with this CDM.
 * The mime type should include a codecs="..." component and if it is video it
 * should also have a resolution=...x... component.
 */
IMediaCDMCapabilities::ESupportResult FHLS_AES128_CDM::SupportsType(const FString& InMimeType)
{
	// Everything is supported.
	return IMediaCDMCapabilities::ESupportResult::Supported;
}

//-----------------------------------------------------------------------------
/**
 * Returns whether or not for a particular media stream format a secure decoder is
 * required to be used.
 */
IMediaCDMCapabilities::ESupportResult FHLS_AES128_CDM::RequiresSecureDecoder(const FString& InMimeType)
{
	return IMediaCDMCapabilities::ESupportResult::SecureDecoderNotRequired;
}

//-----------------------------------------------------------------------------
/**
 * Converts cipher name to enum.
 */
EHLSEncryptionScheme FHLS_AES128_CDM::GetCommonSchemeFromCipher(const FString& InCipherName)
{
	if (InCipherName.Equals(TEXT("cenc"), ESearchCase::IgnoreCase))
	{
		return EHLSEncryptionScheme::Cenc;
	}
	else if (InCipherName.Equals(TEXT("cbcs"), ESearchCase::IgnoreCase))
	{
		return EHLSEncryptionScheme::Cbcs;
	}
	// `cbc7` is not official. We use it internally.
	else if (InCipherName.Equals(TEXT("cbc7"), ESearchCase::IgnoreCase))
	{
		return EHLSEncryptionScheme::Cbc7;
	}
	return EHLSEncryptionScheme::Unsupported;
}

EHLSEncryptionScheme FHLS_AES128_CDM::GetCommonSchemeFromCipher(uint32 InCipher4CC)
{
	#define MAKE_4CC(a,b,c,d) (uint32)(((uint32)a << 24) | ((uint32)b << 16) | ((uint32)c << 8) | ((uint32)d))
	if (InCipher4CC == MAKE_4CC('c', 'e', 'n', 'c'))
	{
		return EHLSEncryptionScheme::Cenc;
	}
	else if (InCipher4CC == MAKE_4CC('c', 'b', 'c', 's'))
	{
		return EHLSEncryptionScheme::Cbcs;
	}
	// `cbc7` is not official. We use it internally.
	else if (InCipher4CC == MAKE_4CC('c', 'b', 'c', '7'))
	{
		return EHLSEncryptionScheme::Cbc7;
	}
	return EHLSEncryptionScheme::Unsupported;
}


/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/


//-----------------------------------------------------------------------------
/**
 * Construct a new client
 */
FHLSDRMClient::FHLSDRMClient()
{
}

//-----------------------------------------------------------------------------
/**
 * Destroy a client
 */
FHLSDRMClient::~FHLSDRMClient()
{
}

//-----------------------------------------------------------------------------
/**
 * Returns the client's current state.
 */
ECDMState FHLSDRMClient::GetState()
{
	FScopeLock lock(&Lock);
	return CurrentState;
}

//-----------------------------------------------------------------------------
/**
 * Returns the client's most recent error message.
 */
FString FHLSDRMClient::GetLastErrorMessage()
{
	FScopeLock lock(&Lock);
	return LastErrorMsg;
}

//-----------------------------------------------------------------------------
/**
 * Initializes the client.
 */
void FHLSDRMClient::Initialize(TSharedPtr<FHLS_AES128_CDM, ESPMode::ThreadSafe> InOwningCDM, IMediaCDM::IPlayerSession* InForPlayerSession, const TArray<IMediaCDM::FCDMCandidate>& InCDMConfigurations)
{
	FScopeLock lock(&Lock);
	OwningCDM = InOwningCDM;
	PlayerSession = InForPlayerSession;
	CDMConfigurations = InCDMConfigurations;
	GetValuesFromConfigurations();
	CurrentState = ECDMState::Idle;
}

//-----------------------------------------------------------------------------
/**
 * Registers an event listener to the client.
 */
void FHLSDRMClient::RegisterEventListener(TWeakPtr<IMediaCDMEventListener, ESPMode::ThreadSafe> InEventListener)
{
	FScopeLock lock(&Lock);
	Listeners.Emplace(InEventListener);

	// Based on the current state we may need to fire events to the new listener right away.
	if (CurrentState == ECDMState::WaitingForKey)
	{
		lock.Unlock();
		FireEvent(IMediaCDMEventListener::ECDMEventType::KeyRequired);
	}
}

//-----------------------------------------------------------------------------
/**
 * Unregisters an event listener from the client.
 */
void FHLSDRMClient::UnregisterEventListener(TWeakPtr<IMediaCDMEventListener, ESPMode::ThreadSafe> InEventListener)
{
	FScopeLock lock(&Lock);
	Listeners.Remove(InEventListener);
}

//-----------------------------------------------------------------------------
/**
 * Fires the given event at all registered event listeners.
 */
void FHLSDRMClient::FireEvent(IMediaCDMEventListener::ECDMEventType InEvent)
{
	TArray<TWeakPtr<IMediaCDMEventListener, ESPMode::ThreadSafe>> CopiedListeners;
	Lock.Lock();
	for(int32 i=0; i<Listeners.Num(); ++i)
	{
		if (Listeners[i].IsValid())
		{
			CopiedListeners.Emplace(Listeners[i]);
		}
		else
		{
			Listeners.RemoveAt(i);
			--i;
		}
	}
	Lock.Unlock();
	TArray<uint8> NoData;
	TSharedPtr<ElectraCDM::IMediaCDMClient, ESPMode::ThreadSafe> This = AsShared();
	for(int32 i=0; i<CopiedListeners.Num(); ++i)
	{
		TSharedPtr<IMediaCDMEventListener, ESPMode::ThreadSafe> Listener = CopiedListeners[i].Pin();
		if (Listener.IsValid())
		{
			Listener->OnCDMEvent(InEvent, This, nullptr, NoData);
		}
	}
}

//-----------------------------------------------------------------------------
/**
 * Prepares the client to fetch a license and fires the event off to the
 * listeners to start the process.
 */
void FHLSDRMClient::PrepareLicenses()
{
	int32 NumToRequest = PrepareKIDsToRequest();
	if (NumToRequest)
	{
		Lock.Lock();
		CurrentState = ECDMState::WaitingForKey;
		Lock.Unlock();
		FireEvent(IMediaCDMEventListener::ECDMEventType::KeyRequired);
	}
	else
	{
		Lock.Lock();
		CurrentState = ECDMState::Ready;
		Lock.Unlock();
	}
}

//-----------------------------------------------------------------------------
/**
 * Overrides the license server URL to the given one.
 * This must happen before calling PrepareLicenses().
 */
void FHLSDRMClient::SetLicenseServerURL(const FString& InLicenseServerURL)
{
	FScopeLock lock(&Lock);
	LicenseServerURLOverride = InLicenseServerURL;
}

//-----------------------------------------------------------------------------
/**
 * Returns the license server URL to which to issue the license request.
 */
void FHLSDRMClient::GetLicenseKeyURL(FString& OutLicenseURL)
{
	FScopeLock lock(&Lock);
	// If the URL has been set explicity from the outside return that one.
	if (LicenseServerURLOverride.IsSet())
	{
		OutLicenseURL = LicenseServerURLOverride.GetValue();
		return;
	}
	// Otherwise, when there are several specified through the AdditionalElements
	// we can return one of them at random.
	// Which means we take the first one.
	if (LicenseServerURLsFromConfigs.Num())
	{
		OutLicenseURL = LicenseServerURLsFromConfigs[0];
		return;
	}
	// Nothing set at all. Clear out the URL in case it contains something.
	OutLicenseURL.Empty();
}

//-----------------------------------------------------------------------------
/**
 * Adds a new KID with license key if the KID is not already known.
 */
void FHLSDRMClient::AddKeyKID(const FHLSKIDKey& InKeyKid)
{
	FScopeLock lock(&Lock);
	for(auto &Key : LicenseKeys)
	{
		if (Key.KID == InKeyKid.KID)
		{
			return;
		}
	}
	LicenseKeys.Add(InKeyKid);
}

//-----------------------------------------------------------------------------
/**
 * Adds a list of new KIDs with license keys when the KID is not already known.
 */
void FHLSDRMClient::AddKeyKIDs(const TArray<FHLSKIDKey>& InKeyKids)
{
	for(auto &KeyKid : InKeyKids)
	{
		AddKeyKID(KeyKid);
	}
}

//-----------------------------------------------------------------------------
/**
 * Prepares the list of KIDs for which a license must be obtained.
 * Licenses the CDM already has will not be requested again.
 */
int32 FHLSDRMClient::PrepareKIDsToRequest()
{
	// We need to get all the KIDs for which we (may) need to acquire a license.
	FScopeLock lock(&Lock);
	PendingRequiredKIDs.Empty();
	check(CDMConfigurations.Num());
	TSharedPtr<FHLS_AES128_CDM, ESPMode::ThreadSafe> CDM = OwningCDM.Pin();
	if (CDMConfigurations.Num())
	{
		for(int32 nCfg=0; nCfg<CDMConfigurations.Num(); ++nCfg)
		{
			for(int32 nKIDs=0; nKIDs<CDMConfigurations[nCfg].DefaultKIDs.Num(); ++nKIDs)
			{
				if (CDMConfigurations[nCfg].DefaultKIDs[nKIDs].Len())
				{
					// Check if the CDM already has a key for this session's KID.
					FString KID = ElectraCDMUtils::StripDashesFromKID(CDMConfigurations[nCfg].DefaultKIDs[nKIDs]);
					TArray<uint8> BinKID;
					FHLSKIDKey KeyKid;
					ElectraCDMUtils::ConvertKIDToBin(BinKID, KID);
					if (CDM.IsValid() && CDM->GetPlayerSessionKey(KeyKid, PlayerSession, BinKID))
					{
						AddKeyKID(KeyKid);
						continue;
					}
					PendingRequiredKIDs.AddUnique(ElectraCDMUtils::ConvertKIDToBase64(KID));
				}
			}
		}
	}
	return PendingRequiredKIDs.Num();
}


//-----------------------------------------------------------------------------
/**
 * Converts encryption scheme string to enum.
 */
EHLSEncryptionScheme FHLSDRMClient::GetCommonSchemeFromConfiguration(const IMediaCDM::FCDMCandidate& InConfiguration)
{
	return FHLS_AES128_CDM::GetCommonSchemeFromCipher(InConfiguration.CommonScheme);
}


//-----------------------------------------------------------------------------
/**
 * Returns the CDM configuration objects matching the given KID.
 */
TArray<IMediaCDM::FCDMCandidate> FHLSDRMClient::GetConfigurationsForKID(const TArray<uint8>& InForKID)
{
	TArray<IMediaCDM::FCDMCandidate> Cfgs;
	FScopeLock lock(&Lock);
	for(int32 nCfg=0; nCfg<CDMConfigurations.Num(); ++nCfg)
	{
		for(int32 nKIDs=0; nKIDs<CDMConfigurations[nCfg].DefaultKIDs.Num(); ++nKIDs)
		{
			if (CDMConfigurations[nCfg].DefaultKIDs[nKIDs].Len())
			{
				TArray<uint8> BinKID;
				ElectraCDMUtils::ConvertKIDToBin(BinKID, ElectraCDMUtils::StripDashesFromKID(CDMConfigurations[nCfg].DefaultKIDs[nKIDs]));
				if (BinKID == InForKID)
				{
					Cfgs.Emplace(CDMConfigurations[nCfg]);
				}
			}
		}
	}
	return Cfgs;
}

//-----------------------------------------------------------------------------
/**
 * Returns the information necessary to make the license request.
 * This includes the system specific blob of data as well as the HTTP method
 * to use and additioanl headers, like the "Content-Type: " header.
 */
void FHLSDRMClient::GetLicenseKeyRequestData(TArray<uint8>& OutKeyRequestData, FString& OutHttpMethod, TArray<FString>& OutHttpHeaders, uint32& OutFlags)
{
	FScopeLock lock(&Lock);
	OutHttpMethod = TEXT("GET");
	// We allow the use of custom key storage.
	OutFlags = EDRMClientFlags::EDRMFlg_AllowCustomKeyStorage;
}

//-----------------------------------------------------------------------------
/**
 * Parses the license key response for keys and provides them to the
 * decrypter instances.
 */
ECDMError FHLSDRMClient::SetLicenseKeyResponseData(void* InEventId, int32 HttpResponseCode, const TArray<uint8>& InKeyResponseData)
{
	bool bSuccess = true;
	TArray<FHLSKIDKey> NewLicenseKeys;
	LastErrorMsg.Empty();
	if (HttpResponseCode == 200)
	{
		if (InKeyResponseData.Num() == 16)
		{
			FScopeLock lock(&Lock);
			for(auto &KID : PendingRequiredKIDs)
			{
				FHLSKIDKey NewKey;
				if (ElectraCDMUtils::Base64UrlDecode(NewKey.KID, KID))
				{
					TArray<IMediaCDM::FCDMCandidate> Configs(GetConfigurationsForKID(NewKey.KID));
					NewKey.EncryptionScheme = Configs.Num() ? GetCommonSchemeFromConfiguration(Configs[0]) : EHLSEncryptionScheme::Unsupported;
					NewKey.Key = InKeyResponseData;
					NewLicenseKeys.Emplace(MoveTemp(NewKey));
				}
			}
		}
		else
		{
			LastErrorMsg = FString::Printf(TEXT("Received bad license key response."));
			bSuccess = false;
		}

		if (bSuccess)
		{
			TSharedPtr<FHLS_AES128_CDM, ESPMode::ThreadSafe> CDM = OwningCDM.Pin();
			if (CDM.IsValid())
			{
				CDM->AddPlayerSessionKeys(PlayerSession, NewLicenseKeys);
			}

			AddKeyKIDs(NewLicenseKeys);
			Lock.Lock();
			CurrentState = ECDMState::Ready;
			Lock.Unlock();

			UpdateKeyWithDecrypters();
			return ECDMError::Success;
		}
	}
	else
	{
		LastErrorMsg = FString::Printf(TEXT("Received bad license key response. HTTP code %d"), HttpResponseCode);
	}
	CurrentState = ECDMState::InvalidKey;
	UpdateStateWithDecrypters(CurrentState);
	return ECDMError::Failure;
}

//-----------------------------------------------------------------------------
/**
 * Creates a decrypter instance.
 * If the license keys have not been obtained yet the decrypter will not be
 * usable until the keys arrive.
 * One decrypter instance is created per elementary stream to decode.
 */
ECDMError FHLSDRMClient::CreateDecrypter(TSharedPtr<IMediaCDMDecrypter, ESPMode::ThreadSafe>& OutDecrypter, const FString& InMimeType)
{
	TSharedPtr<FHLSDRMDecrypter, ESPMode::ThreadSafe> NewDec = MakeShared<FHLSDRMDecrypter, ESPMode::ThreadSafe>();
	NewDec->Initialize(InMimeType);

	FScopeLock lock(&Lock);

	// The initial state of the decrypter is the same as the one of the client.
	NewDec->SetState(CurrentState);
	NewDec->SetLastErrorMessage(LastErrorMsg);
	// If ready set the key with the decrypter.
	if (CurrentState == ECDMState::Ready)
	{
		NewDec->SetLicenseKeys(LicenseKeys);
	}

	RemoveStaleDecrypters();
	Decrypters.Emplace(NewDec);
	OutDecrypter = NewDec;
	return ECDMError::Success;
}


//-----------------------------------------------------------------------------
/**
 * Removes decrypters that the application no longer uses.
 */
void FHLSDRMClient::RemoveStaleDecrypters()
{
	for(int32 i=0; i<Decrypters.Num(); ++i)
	{
		if (!Decrypters[i].IsValid())
		{
			Decrypters.RemoveAt(i);
			--i;
		}
	}
}

//-----------------------------------------------------------------------------
/**
 * Updates all this client's decrypters with the new set of license keys.
 */
void FHLSDRMClient::UpdateKeyWithDecrypters()
{
	FScopeLock lock(&Lock);
	RemoveStaleDecrypters();
	for(int32 i=0; i<Decrypters.Num(); ++i)
	{
		TSharedPtr<FHLSDRMDecrypter, ESPMode::ThreadSafe> Decrypter = Decrypters[i].Pin();
		Decrypter->SetLicenseKeys(LicenseKeys);
	}
}

//-----------------------------------------------------------------------------
/**
 * Sets the state of all this client's decrypters to the given state.
 * This is called in cases of license errors to make all instances fail.
 */
void FHLSDRMClient::UpdateStateWithDecrypters(ECDMState InNewState)
{
	FScopeLock lock(&Lock);
	RemoveStaleDecrypters();
	for(int32 i=0; i<Decrypters.Num(); ++i)
	{
		TSharedPtr<FHLSDRMDecrypter, ESPMode::ThreadSafe> Decrypter = Decrypters[i].Pin();
		Decrypter->SetLastErrorMessage(LastErrorMsg);
		Decrypter->SetState(InNewState);
	}
}

//-----------------------------------------------------------------------------
/**
 * Extracts relevant information from the AdditionalElements
 */
void FHLSDRMClient::GetValuesFromConfigurations()
{
	FScopeLock lock(&Lock);
	for(auto &Config : CDMConfigurations)
	{
		if (Config.AdditionalElements.IsEmpty())
		{
			continue;
		}

		TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(Config.AdditionalElements);
		TSharedPtr<FJsonObject> ConfigJSON;
		if (FJsonSerializer::Deserialize(Reader, ConfigJSON))
		{
			// Try to get the URI
			FString URI;
			if (ConfigJSON->TryGetStringField(HLSKeyParamNames::URI, URI))
			{
				LicenseServerURLsFromConfigs.AddUnique(MoveTemp(URI));
			}
			else
			{
				UE_LOGF(LogElectraCDM, Log, "Required URI not found in configuration object");
			}
		}
		else
		{
			LastErrorMsg = TEXT("Could not parse additional configuration element.");
			CurrentState = ECDMState::ConfigurationError;
			return;
		}
	}
}





//-----------------------------------------------------------------------------
/**
 * Creates a new decrypter instance.
 */
FHLSDRMDecrypter::FHLSDRMDecrypter()
{
}

//-----------------------------------------------------------------------------
/**
 * Destroys a decrypter instance.
 */
FHLSDRMDecrypter::~FHLSDRMDecrypter()
{
}

//-----------------------------------------------------------------------------
/**
 * Initializes a decrypter instance to default state.
 */
void FHLSDRMDecrypter::Initialize(const FString& InMimeType)
{
	FScopeLock lock(&Lock);
	CurrentState = ECDMState::Idle;
	KeyDecrypters.Empty();
	MimeType = InMimeType;
}

//-----------------------------------------------------------------------------
/**
 * Updates the valid license keys with this decrypter instance.
 * All currently active keys are removed and replaced with the new ones.
 */
void FHLSDRMDecrypter::SetLicenseKeys(const TArray<FHLSKIDKey>& InLicenseKeys)
{
	FScopeLock lock(&Lock);
	LicenseKeys = InLicenseKeys;
	Reinitialize();
}

//-----------------------------------------------------------------------------
/**
 * Returns the current decrypter's state.
 */
ECDMState FHLSDRMDecrypter::GetState()
{
	FScopeLock lock(&Lock);
	// This is the amalgamated state of all the internal decrypters per
	// active key.
	return CurrentState;
}

//-----------------------------------------------------------------------------
/**
 * Returns the most recent error message.
 */
FString FHLSDRMDecrypter::GetLastErrorMessage()
{
	FScopeLock lock(&Lock);
	return LastErrorMsg;
}

//-----------------------------------------------------------------------------
/**
 * Sets a new state to this decrypter and all its currently active key decrypters.
 */
void FHLSDRMDecrypter::SetState(ECDMState InNewState)
{
	FScopeLock lock(&Lock);
	CurrentState = InNewState;
	for(int32 i=0; i<KeyDecrypters.Num(); ++i)
	{
		KeyDecrypters[i].State = InNewState;
	}
}

//-----------------------------------------------------------------------------
/**
 * Updates the last error message.
 */
void FHLSDRMDecrypter::SetLastErrorMessage(const FString& InNewErrorMessage)
{
	FScopeLock lock(&Lock);
	LastErrorMsg = InNewErrorMessage;
}


//-----------------------------------------------------------------------------
/**
 * Called by the application with PSSH box data to update the current set of
 * key IDs when key rotation is used.
 */
ECDMError FHLSDRMDecrypter::UpdateInitDataFromPSSH(const TArray<uint8>& InPSSHData)
{
	return ECDMError::NotSupported;
}

ECDMError FHLSDRMDecrypter::UpdateInitDataFromMultiplePSSH(const TArray<TArray<uint8>>& InPSSHData)
{
	return ECDMError::NotSupported;
}

//-----------------------------------------------------------------------------
/**
 * Update from a URL and additional scheme specific elements.
 */
ECDMError FHLSDRMDecrypter::UpdateFromURL(const FString& InURL, const FString& InAdditionalElements)
{
	return ECDMError::NotSupported;
}

//-----------------------------------------------------------------------------
/**
 * Locates the decrypter for the given key ID.
 */
FHLSDRMDecrypter::FKeyDecrypter* FHLSDRMDecrypter::GetDecrypterForKID(const TArray<uint8>& KID)
{
	// Note: The critical section must be locked already!
	for(int32 i=0; i<KeyDecrypters.Num(); ++i)
	{
		if (KID == KeyDecrypters[i].KIDKey.KID)
		{
			// Decrypter needs to be ready.
			if (KeyDecrypters[i].State == ECDMState::Ready)
			{
				return &KeyDecrypters[i];
			}
		}
	}
	return nullptr;
}

//-----------------------------------------------------------------------------
/**
 * Reinitializes the decrypter to its starting state.
 */
void FHLSDRMDecrypter::Reinitialize()
{
	FScopeLock lock(&Lock);

	KeyDecrypters.Empty();
	LastErrorMsg.Empty();

	CurrentState = ECDMState::Ready;
	for(int32 i=0; i<LicenseKeys.Num(); ++i)
	{
		FKeyDecrypter& kd = KeyDecrypters.AddDefaulted_GetRef();
		kd.KIDKey = LicenseKeys[i];
		kd.Decrypter = ElectraCDM::IStreamDecrypterAES128::Create();
		kd.State = ECDMState::Ready;
		kd.bIsInitialized = false;
	}
}

//-----------------------------------------------------------------------------
/**
 * Decrypts data in place according to the encrypted sample information.
 */
ECDMError FHLSDRMDecrypter::DecryptInPlace(uint8* InOutData, int32 InNumDataBytes, const FMediaCDMSampleInfo& InSampleInfo)
{
	FScopeLock lock(&Lock);
	LastErrorMsg.Empty();
	FHLSDRMDecrypter::FKeyDecrypter* DecrypterState = GetDecrypterForKID(InSampleInfo.DefaultKID);
	TSharedPtr<ElectraCDM::IStreamDecrypterAES128, ESPMode::ThreadSafe> Decrypter = DecrypterState ? DecrypterState->Decrypter : nullptr;
	if (Decrypter.IsValid() && DecrypterState)
	{
		ElectraCDM::IStreamDecrypterAES128::EResult Result;

		EHLSEncryptionScheme SchemeFromMedia = FHLS_AES128_CDM::GetCommonSchemeFromCipher(InSampleInfo.Scheme4CC);
		EHLSEncryptionScheme SchemeFromKID = DecrypterState->KIDKey.EncryptionScheme;

		EHLSEncryptionScheme SchemeToUse = SchemeFromMedia != EHLSEncryptionScheme::Unsupported ? SchemeFromMedia : SchemeFromKID;

		// "cenc" scheme? (AES-128 CTR)
		if (SchemeToUse == EHLSEncryptionScheme::Cenc)
		{
			if (!DecrypterState->bIsInitialized)
			{
				Result = Decrypter->CTRInit(DecrypterState->KIDKey.Key);
				if ((DecrypterState->bIsInitialized = Result == ElectraCDM::IStreamDecrypterAES128::EResult::Ok) == false)
				{
					DecrypterState->State = ECDMState::InvalidKey;
					CurrentState = ECDMState::InvalidKey;
					LastErrorMsg = TEXT("Invalid key");
					return ECDMError::Failure;
				}
			}

			if (Decrypter->CTRSetIV(InSampleInfo.IV) != ElectraCDM::IStreamDecrypterAES128::EResult::Ok)
			{
				LastErrorMsg = TEXT("Bad IV");
				return ECDMError::Failure;
			}
			if (InSampleInfo.SubSamples.Num() == 0)
			{
				Decrypter->CTRDecryptInPlace(InOutData, InNumDataBytes);
				return ECDMError::Success;
			}
			else
			{
				for(int32 i=0; i<InSampleInfo.SubSamples.Num(); ++i)
				{
					InOutData += InSampleInfo.SubSamples[i].NumClearBytes;
					if (InSampleInfo.SubSamples[i].NumEncryptedBytes)
					{
						Decrypter->CTRDecryptInPlace(InOutData, InSampleInfo.SubSamples[i].NumEncryptedBytes);
					}
					InOutData += InSampleInfo.SubSamples[i].NumEncryptedBytes;
				}
				return ECDMError::Success;
			}
		}
		// "cbcs" scheme? (AES-128 CBC)
		else if (SchemeToUse == EHLSEncryptionScheme::Cbcs)
		{
			int32 NumBytesDecrypted = 0;

			auto DecryptPatternBlock = [&](int32 BlocksToGo) -> void
			{
				while(BlocksToGo > 0)
				{
					int32 NumEnc = BlocksToGo >= InSampleInfo.Pattern.CryptByteBlock ? InSampleInfo.Pattern.CryptByteBlock : BlocksToGo;
					/*Result =*/ Decrypter->CBCDecryptInPlace(NumBytesDecrypted, InOutData, NumEnc * 16, false);
					InOutData += (NumEnc + InSampleInfo.Pattern.SkipByteBlock) * 16;
					BlocksToGo -= InSampleInfo.Pattern.CryptByteBlock + InSampleInfo.Pattern.SkipByteBlock;
				}
			};

			if (InSampleInfo.SubSamples.Num() == 0)
			{
				Result = DecrypterState->Decrypter->CBCInit(DecrypterState->KIDKey.Key, &InSampleInfo.IV);
				if (Result != ElectraCDM::IStreamDecrypterAES128::EResult::Ok)
				{
					LastErrorMsg = TEXT("Bad key or IV length");
					return ECDMError::Failure;
				}

				// Entire sample is encrypted.
				if (InSampleInfo.Pattern.CryptByteBlock == 0 && InSampleInfo.Pattern.SkipByteBlock == 0)
				{
					/*Result =*/ Decrypter->CBCDecryptInPlace(NumBytesDecrypted, InOutData, InNumDataBytes & ~15, false);
				}
				else
				{
					DecryptPatternBlock(InNumDataBytes / 16);
				}
				return ECDMError::Success;
			}
			else
			{
				for(int32 i=0; i<InSampleInfo.SubSamples.Num(); ++i)
				{
					InOutData += InSampleInfo.SubSamples[i].NumClearBytes;

					if (InSampleInfo.SubSamples[i].NumEncryptedBytes)
					{
						// cbcs encryption is restarted with every sub-sample.
						if (InSampleInfo.IV.Num() == 16)
						{
							Result = DecrypterState->Decrypter->CBCInit(DecrypterState->KIDKey.Key, &InSampleInfo.IV);
						}
						else if (InSampleInfo.IV.Num() < 16)
						{
							TArray<uint8> paddedIV(InSampleInfo.IV);
							paddedIV.InsertZeroed(paddedIV.Num(), 16-paddedIV.Num());
							Result = DecrypterState->Decrypter->CBCInit(DecrypterState->KIDKey.Key, &paddedIV);
						}
						else
						{
							Result = ElectraCDM::IStreamDecrypterAES128::EResult::BadIVLength;
						}
						if (Result != ElectraCDM::IStreamDecrypterAES128::EResult::Ok)
						{
							LastErrorMsg = TEXT("Bad key or IV length");
							return ECDMError::Failure;
						}

						if (InSampleInfo.Pattern.CryptByteBlock == 0 && InSampleInfo.Pattern.SkipByteBlock == 0)
						{
							/*Result =*/ Decrypter->CBCDecryptInPlace(NumBytesDecrypted, InOutData, InSampleInfo.SubSamples[i].NumEncryptedBytes & ~15, false);
							InOutData = InOutData + InSampleInfo.SubSamples[i].NumEncryptedBytes;
						}
						else
						{
							uint8* EncDataStart = InOutData;
							DecryptPatternBlock((int32) InSampleInfo.SubSamples[i].NumEncryptedBytes / 16);
							// The number of encrypted bytes in the subsample is not necessarily a 16B multiple.
							// If not, the last few bytes are not encrypted and as such are not touched by
							// decryption. We need to advance the pointer to the data to the actual number of
							// bytes - NOT the number of bytes that were decrypted! - to get to the next subsample.
							InOutData = EncDataStart + InSampleInfo.SubSamples[i].NumEncryptedBytes;
						}
					}
				}
				return ECDMError::Success;
			}
		}
		else
		{
			LastErrorMsg = TEXT("Unsupported encryption scheme for KID");
			return ECDMError::Failure;
		}
	}
	LastErrorMsg = TEXT("No valid decrypter found for KID");
	return ECDMError::Failure;
}


bool FHLSDRMDecrypter::IsBlockStreamDecrypter()
{
	FScopeLock lock(&Lock);
	// This depends on the encryption method
	return MimeType.Equals(TEXT("cbc7"));
}

ECDMError FHLSDRMDecrypter::BlockStreamDecryptStart(IStreamDecryptHandle*& OutStreamDecryptContext, const FMediaCDMSampleInfo& InSampleInfo)
{
	OutStreamDecryptContext = nullptr;

	FScopeLock lock(&Lock);
	if (!IsBlockStreamDecrypter())
	{
		LastErrorMsg = TEXT("Not a block stream decrypter");
		return ECDMError::CipherModeMismatch;
	}

	FHLSDRMDecrypter::FKeyDecrypter* DecrypterState = GetDecrypterForKID(InSampleInfo.DefaultKID);
	TSharedPtr<ElectraCDM::IStreamDecrypterAES128, ESPMode::ThreadSafe> Decrypter = DecrypterState ? DecrypterState->Decrypter : nullptr;
	if (Decrypter.IsValid() && DecrypterState)
	{
		if (!DecrypterState->bIsInitialized)
		{
			ElectraCDM::IStreamDecrypterAES128::EResult Result = Decrypter->CBCInit(DecrypterState->KIDKey.Key, &InSampleInfo.IV);
			if ((DecrypterState->bIsInitialized = Result == ElectraCDM::IStreamDecrypterAES128::EResult::Ok) == false)
			{
				DecrypterState->State = ECDMState::InvalidKey;
				CurrentState = ECDMState::InvalidKey;
				LastErrorMsg = TEXT("Invalid key");
				return ECDMError::Failure;
			}
		}
		FBlockDecrypterHandle* Handle = new FBlockDecrypterHandle;
		Handle->BlockSize = Decrypter->CBCGetEncryptionDataSize(1);
		Handle->KID = InSampleInfo.DefaultKID;
		OutStreamDecryptContext = reinterpret_cast<IStreamDecryptHandle*>(Handle);
		return ECDMError::Success;
	}
	else
	{
		LastErrorMsg = TEXT("No valid decrypter found for KID");
		return ECDMError::Failure;
	}
}

ECDMError FHLSDRMDecrypter::BlockStreamDecryptInPlace(IStreamDecryptHandle* InOutStreamDecryptContext, int32& OutNumBytesDecrypted, uint8* InOutData, int32 InNumDataBytes, bool bIsLastBlock)
{
	FBlockDecrypterHandle* Handle = reinterpret_cast<FBlockDecrypterHandle*>(InOutStreamDecryptContext);
	if (Handle)
	{
		FScopeLock lock(&Lock);
		FHLSDRMDecrypter::FKeyDecrypter* DecrypterState = GetDecrypterForKID(Handle->KID);
		TSharedPtr<ElectraCDM::IStreamDecrypterAES128, ESPMode::ThreadSafe> Decrypter = DecrypterState ? DecrypterState->Decrypter : nullptr;
		if (Decrypter.IsValid() && DecrypterState && DecrypterState->bIsInitialized)
		{
			ElectraCDM::IStreamDecrypterAES128::EResult Result = Decrypter->CBCDecryptInPlace(OutNumBytesDecrypted, InOutData, InNumDataBytes, bIsLastBlock);
			if (Result == ElectraCDM::IStreamDecrypterAES128::EResult::Ok)
			{
				return ECDMError::Success;
			}
			LastErrorMsg = FString::Printf(TEXT("Failed to decrypt (%d)"), (int32)Result);
			return ECDMError::Failure;
		}
		LastErrorMsg = TEXT("Invalid or incorrect decrypter");
		return ECDMError::Failure;
	}
	LastErrorMsg = TEXT("Invalid context passed");
	return ECDMError::Failure;
}

ECDMError FHLSDRMDecrypter::BlockStreamDecryptEnd(IStreamDecryptHandle* InStreamDecryptContext)
{
	FBlockDecrypterHandle* Handle = reinterpret_cast<FBlockDecrypterHandle*>(InStreamDecryptContext);
	if (Handle)
	{
		FHLSDRMDecrypter::FKeyDecrypter* DecrypterState = GetDecrypterForKID(Handle->KID);
		TSharedPtr<ElectraCDM::IStreamDecrypterAES128, ESPMode::ThreadSafe> Decrypter = DecrypterState ? DecrypterState->Decrypter : nullptr;
		if (Decrypter.IsValid() && DecrypterState)
		{
			DecrypterState->bIsInitialized = false;
		}
		delete Handle;
	}
	return ECDMError::Success;
}


}
