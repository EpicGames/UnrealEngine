// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ElectraCDM.h"
#include "ElectraCDMSystem.h"

namespace ElectraCDM
{

class IHLS_AES128_CDM : public IMediaCDMSystem
{
public:
	static void RegisterWith(IMediaCDM& InDRMManager);
	virtual ~IHLS_AES128_CDM() = default;
	virtual FString GetLastErrorMessage() = 0;
	virtual const TArray<FString>& GetSchemeIDs() = 0;

	virtual void GetCDMCustomJSONPrefixes(FString& OutAttributePrefix, FString& OutTextPropertyName, bool& bOutNoNamespaces) = 0;

	virtual TSharedPtr<IMediaCDMCapabilities, ESPMode::ThreadSafe> GetCDMCapabilities(const FString& InValue, const FString& InAdditionalElements) = 0;
	virtual ECDMError CreateDRMClient(TSharedPtr<IMediaCDMClient, ESPMode::ThreadSafe>& OutClient, IMediaCDM::IPlayerSession* InForPlayerSession, const TArray<IMediaCDM::FCDMCandidate>& InCandidates) = 0;
	virtual ECDMError ReleasePlayerSessionKeys(IMediaCDM::IPlayerSession* PlayerSession) = 0;
};

}
