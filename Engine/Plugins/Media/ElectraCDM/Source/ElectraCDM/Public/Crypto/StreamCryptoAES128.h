// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <CoreMinimal.h>

#define UE_API ELECTRACDM_API

namespace ElectraCDM
{


/**
 *
 */
class IStreamDecrypterAES128
{
public:
	static UE_API TSharedPtr<IStreamDecrypterAES128, ESPMode::ThreadSafe> Create();
	virtual ~IStreamDecrypterAES128() = default;

	enum class EResult
	{
		Ok,
		NotInitialized,
		BadKeyLength,
		BadIVLength,
		BadDataLength,
		InvalidArg,
		BadHexChar
	};
	static UE_API const TCHAR* GetResultText(EResult ResultCode);

	static UE_API EResult ConvHexStringToBin(TArray<uint8>& OutBinData, const char* InHexString);
	static UE_API void MakePaddedIVFromUInt64(TArray<uint8>& OutBinData, uint64 lower64Bits);

	virtual EResult CBCInit(const TArray<uint8>& Key, const TArray<uint8>* OptionalIV=nullptr) = 0;
	virtual EResult CBCDecryptInPlace(int32& OutNumBytes, uint8* InOutData, int32 NumBytes16, bool bIsFinalBlock) = 0;
	virtual int32 CBCGetEncryptionDataSize(int32 PlaintextSize) = 0;
	virtual EResult CBCEncryptInPlace(int32& OutNumBytes, uint8* InOutData, int32 NumBytes, bool bIsFinalData) = 0;

	virtual EResult CTRInit(const TArray<uint8>& Key) = 0;
	virtual EResult CTRSetKey(const TArray<uint8>& Key) = 0;
	virtual EResult CTRSetIV(const TArray<uint8>& IV) = 0;
	virtual EResult CTRDecryptInPlace(uint8* InOutData, int32 InNumBytes) = 0;
};


} // namespace ElectraCDM


#undef UE_API
