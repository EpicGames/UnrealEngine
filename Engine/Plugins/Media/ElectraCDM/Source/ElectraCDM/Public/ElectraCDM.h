// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <CoreMinimal.h>
#include <Containers/Array.h>
#include "ElectraCDMError.h"

#define UE_API ELECTRACDM_API

namespace ElectraCDM
{

/*
	Note #1:
		The mime type may (need to) be followed by a codec as per RFC 6381.
		In addition, for video formats, the codec should be followed by resolution=WxH

		Example for the MIME type:
			video/mp4; codecs="avc1.640028"; resolution=1920x1080
			audio/mp4; codecs="mp4a.40.2"

	Note #2:
		Parameters are as with ISO/IEC 23009-1:2019 where:
			"InCDMScheme" refers to the ContentProtection@schemeIdUri attribute
			"InValue" refers to the ContentProtection@value attribute
			"InAdditionalElements" is a JSON string of all other ContentProtection element attributes and all child elements with their respective attributes.
*/


class IMediaCDMSystem;
class IMediaCDMClient;

class IMediaCDMCapabilities
{
public:
	virtual ~IMediaCDMCapabilities() = default;

	enum class ESupportResult
	{
		// The media type is supported.
		Supported,
		// The media type is not supported.
		NotSupported,
		// The specified MIME type needs to provide the "codecs" information
		// to determine whether or not the type is supported.
		CodecRequired,
		// The specified MIME type needs to provide feature information to
		// determine whether or not the type is supported. (eg.: resolution=WxH)
		FeatureRequired,
		// Failed to parse the MIME type string. Check if it is of correct syntax.
		BadMIMEType,

		// Secure decoder required.
		SecureDecoderRequired,
		// Secure decoder not required.
		SecureDecoderNotRequired,
	};

	// Returns if a specified cipher (eg. "cenc" or "cbcs") is supported by this CDM.
	virtual ESupportResult SupportsCipher(const FString& InCipherType) = 0;

	// Returns if a specified media type is supported by this CDM.
	virtual ESupportResult SupportsType(const FString& InMimeType) = 0;

	// Returns if a secure decoder is required for the specified media type.
	// This SHOULD be able to return the proper answer WITHOUT having acquired a
	// license since this is may be used to select/deselect streams upfront.
	virtual ESupportResult RequiresSecureDecoder(const FString& InMimeType) = 0;
};


class IMediaCDM
{
public:
	static UE_API IMediaCDM& Get();

	//-------------------------------------------------------------------------
	// CDM registration
	//
	virtual void RegisterCDM(TWeakPtr<IMediaCDMSystem, ESPMode::ThreadSafe> InCDMSystem) = 0;


	//-------------------------------------------------------------------------
	// Obtains the CDM system's preference on how attributes and text properties should be prefixed in the JSON string
	// passed to GetCDMCapabilitiesForScheme(..., InAdditionalElements) or in the FCDMCandidate.AdditionalElements.
	virtual void GetCDMCustomJSONPrefixes(const FString& InCDMScheme, const FString& InValue, FString& OutAttributePrefix, FString& OutTextPropertyName, bool& bOutNoNamespaces) = 0;


	//-------------------------------------------------------------------------
	// CDM system capabilities
	//
	// Returns an interface to a CDM system's capabilities. The system is identified through its unique scheme id (a UUID).
	// If there is no CDM system for the given scheme a nullptr is returned.
	virtual TSharedPtr<IMediaCDMCapabilities, ESPMode::ThreadSafe> GetCDMCapabilitiesForScheme(const FString& InCDMScheme, const FString& InValue, const FString& InAdditionalElements) = 0;


	//-------------------------------------------------------------------------
	// State
	//
	virtual FString GetLastErrorMessage() = 0;

	
	//-------------------------------------------------------------------------
	// Client
	//
	
	// A "player session" is a handle created by and for one instance of a customer (ie. a media player)
	// This session will associate registered clients with the license keys they are using.
	struct IPlayerSession;
	virtual IPlayerSession* CreatePlayerSessionID() = 0;
	// Releases the specified player session and its associated keys.
	virtual void ReleasePlayerSessionID(IPlayerSession* InPlayerSessionToRelease) = 0;

	struct FCDMCandidate
	{
		FString SchemeId;
		FString Value;
		FString CommonScheme;
		FString AdditionalElements;
		TArray<FString> DefaultKIDs;
	};
	// Create a single client from all the given candidates. Candidates are those whose capabilities said they are able to decrypt
	// a particular stream. The object of this method is to settle on one CDM system (in case several claimed capability to decrypt).
	// Returns Success or Failure if different types of streams would be using a different type of CDM.
	virtual ECDMError CreateDRMClient(TSharedPtr<IMediaCDMClient, ESPMode::ThreadSafe>& OutClient, IPlayerSession* InForPlayerSession, const TArray<FCDMCandidate>& InCandidates) = 0;

	// Releases all license keys associated by the specified session.
	// Must be called before calling ReleasePlayerSessionID().
	virtual ECDMError ReleasePlayerSessionKeys(IPlayerSession* InPlayerSession) = 0;

protected:
	virtual ~IMediaCDM() = default;
};


}

#undef UE_API
