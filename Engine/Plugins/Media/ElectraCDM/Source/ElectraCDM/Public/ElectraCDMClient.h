// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <CoreMinimal.h>
#include "ElectraCDMError.h"
#include "ElectraEncryptedSampleInfo.h"


namespace ElectraCDM
{

class IMediaCDMClient;
class IMediaCDMDecrypter;


/**
 * Event listener to be registered with a CDM client.
 * This must be implemented by the application to be notified about the CDM client's needs
 * that need to be satisfied, like performing license acquisition.
 */
class IMediaCDMEventListener
{
public:
	virtual ~IMediaCDMEventListener() = default;

	enum class ECDMEventType
	{
		ProvisionRequired,
		KeyRequired,
		KeyExpired
	};

	/**
	 * An event triggered by a CDM system that needs to be processed by the applicate.
	 * Please note that the InDrmClient may be an intermediate client your application did NOT create!
	 * Do not perform checks on the client pointer.
	 * Also do not use this client to perform any operation other than to return the requested data,
	 * if the event is a data request.
	 * You have to return the opaque InEventId in the reply (see SetLicenseKeyResponseData()).
	 */
	virtual void OnCDMEvent(ECDMEventType InEventType, TSharedPtr<ElectraCDM::IMediaCDMClient, ESPMode::ThreadSafe> InDrmClient, void* InEventId, const TArray<uint8>& InCustomData) = 0;
};





/**
 * CDM client
 * This interfaces the application with the CDM system.
 * An application can use multiple clients if necessary.
 */
class IMediaCDMClient
{
public:
	virtual ~IMediaCDMClient() = default;


	//-------------------------------------------------------------------------
	// Application event listener methods.
	//
	// Registers an application event listener to receive OnCDMEvent() messages.
	virtual void RegisterEventListener(TWeakPtr<IMediaCDMEventListener, ESPMode::ThreadSafe> InEventListener) = 0;
	// Unregisters a previously registered event listener.
	virtual void UnregisterEventListener(TWeakPtr<IMediaCDMEventListener, ESPMode::ThreadSafe> InEventListener) = 0;


	//-------------------------------------------------------------------------
	// State
	//
	virtual ECDMState GetState() = 0;

	virtual FString GetLastErrorMessage() = 0;


	//-------------------------------------------------------------------------
	// License acquisition
	//
	// Prepares the client for license acquisition based on the configuration passed in IMediaCDM::CreateDRMClient()
	virtual void PrepareLicenses() = 0;

	enum EDRMClientFlags
	{
		EDRMFlg_None = 0,
		EDRMFlg_AllowCustomKeyStorage = 1U << 31
	};

	// Overrides the URL to the license server that may have been set in the initial configuration data.
	virtual void SetLicenseServerURL(const FString& InLicenseServerURL) = 0;
	// Returns the URL to the license server, either from configuration data or the overrride.
	virtual void GetLicenseKeyURL(FString& OutLicenseURL) = 0;
	// Returns the data to send to the license server as well as the HTTP method and additional HTTP headers.
	virtual void GetLicenseKeyRequestData(TArray<uint8>& OutKeyRequestData, FString& OutHttpMethod, TArray<FString>& OutHttpHeaders, uint32& OutFlags) = 0;
	// Sets the license server's response data. This will, if successful, also update any already existing decrypters with the keys.
	// The EventId must be the one received in OnCDMEvent().
	virtual ECDMError SetLicenseKeyResponseData(void* InEventId, int32 HttpResponseCode, const TArray<uint8>& InKeyResponseData) = 0;


	//-------------------------------------------------------------------------
	// Decrypter
	//
	// Creates a decrypter for the specified MIME type.
	virtual ECDMError CreateDecrypter(TSharedPtr<IMediaCDMDecrypter, ESPMode::ThreadSafe>& OutDecrypter, const FString& InMimeType) = 0;
};



/**
 * Decrypter instance.
 */
class IMediaCDMDecrypter
{
public:
	virtual ~IMediaCDMDecrypter() = default;

	//-------------------------------------------------------------------------
	// State
	//
	virtual ECDMState GetState() = 0;
	virtual FString GetLastErrorMessage() = 0;


	//-------------------------------------------------------------------------
	// License update
	//
	// Update from PSSH boxes encountered in the stream.
	virtual ECDMError UpdateInitDataFromPSSH(const TArray<uint8>& InPSSHData) = 0;
	virtual ECDMError UpdateInitDataFromMultiplePSSH(const TArray<TArray<uint8>>& InPSSHData) = 0;

	// Update from a URL and additional scheme specific elements.
	virtual ECDMError UpdateFromURL(const FString& InURL, const FString& InAdditionalElements) = 0;


	//-------------------------------------------------------------------------
	// Decryption
	//

	// Returns if this decrypter needs to work in block streaming mode (based on the encryption settings).
	virtual bool IsBlockStreamDecrypter() = 0;

	// Reinitializes the decrypter to its starting state. This should not normally be called unless an error occurred.
	virtual void Reinitialize() = 0;

	// Decrypt a non-block oriented cipher (eg. AES 128 CTR) in place.
	virtual ECDMError DecryptInPlace(uint8* InOutData, int32 InNumDataBytes, const FMediaCDMSampleInfo& InSampleInfo) = 0;

	// Decrypt a streaming cipher that requires input of fixed cipher block lengths and possibly padding at the end (eg. AES 128 CBC with PKCS#7 padding).
	// This needs a running state during decryption that needs to be created and released.
	struct IStreamDecryptHandle
	{
		int32 BlockSize = 0;
	};
	virtual ECDMError BlockStreamDecryptStart(IStreamDecryptHandle*& OutStreamDecryptContext, const FMediaCDMSampleInfo& InSampleInfo) = 0;
	virtual ECDMError BlockStreamDecryptInPlace(IStreamDecryptHandle* InOutStreamDecryptContext, int32& OutNumBytesDecrypted, uint8* InOutData, int32 InNumDataBytes, bool bIsLastBlock) = 0;
	virtual ECDMError BlockStreamDecryptEnd(IStreamDecryptHandle* InStreamDecryptContext) = 0;
};

}

