// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <CoreMinimal.h>

namespace ElectraCDM
{


enum class ECDMError
{
	Success,
	Failure,
	NotSupported,
	CipherModeMismatch
};


enum class ECDMState
{
	Idle,
	WaitingForKey,
	Ready,
	InvalidKey,
	KeyExpired,
	ConfigurationError
};


}

