// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <CoreMinimal.h>
#include <Containers/Array.h>
#include "ElectraCDM.h"

namespace ElectraCDM
{

class IMediaCDMSystem
{
public:
	virtual FString GetLastErrorMessage() = 0;

	virtual const TArray<FString>& GetSchemeIDs() = 0;

	virtual void GetCDMCustomJSONPrefixes(FString& OutAttributePrefix, FString& OutTextPropertyName, bool& bOutNoNamespaces) = 0;

	virtual TSharedPtr<IMediaCDMCapabilities, ESPMode::ThreadSafe> GetCDMCapabilities(const FString& InValue, const FString& InAdditionalElements) = 0;

	virtual ECDMError CreateDRMClient(TSharedPtr<IMediaCDMClient, ESPMode::ThreadSafe>& OutClient, IMediaCDM::IPlayerSession* InForPlayerSession, const TArray<IMediaCDM::FCDMCandidate>& InCandidates) = 0;
	virtual ECDMError ReleasePlayerSessionKeys(IMediaCDM::IPlayerSession* PlayerSession) = 0;

protected:
	virtual ~IMediaCDMSystem() = default;
};

}

