// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <CoreMinimal.h>
#include <Containers/Array.h>

namespace ElectraCDM
{


struct FMediaCDMSampleInfo
{
	struct FSubSample
	{
		uint16 NumClearBytes = 0;
		uint32 NumEncryptedBytes = 0;
	};
	struct FCryptSkipPattern
	{
		uint8 PatternType = 0;
		uint8 CryptByteBlock = 0;
		uint8 SkipByteBlock = 0;
	};
	TArray<uint8> DefaultKID;
	TArray<TArray<uint8>> KIDs;
	TArray<uint8> IV;
	TArray<FSubSample> SubSamples;
	FCryptSkipPattern Pattern;
	uint32 Scheme4CC = 0;
};



}

