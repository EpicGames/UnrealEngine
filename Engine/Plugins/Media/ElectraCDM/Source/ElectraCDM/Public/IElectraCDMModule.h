// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Modules/ModuleInterface.h"

/**
 * Interface for the `ElectraCDM` module.
 */
class IElectraCDMModule
	: public IModuleInterface
{
public:

	virtual ~IElectraCDMModule() = default;

private:
};

